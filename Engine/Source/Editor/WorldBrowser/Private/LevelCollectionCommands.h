// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "InputCoreTypes.h"
#include "Framework/Commands/InputChord.h"
#include "Styling/AppStyle.h"
#include "Framework/Commands/Commands.h"

#define LOCTEXT_NAMESPACE "LevelCollectionCommands"

/** The set of commands supported by the WorldView */
class FLevelCollectionCommands 
	: public TCommands<FLevelCollectionCommands>
{

public:

	/** FLevelCollectionCommands Constructor */
	FLevelCollectionCommands() 
		: TCommands<FLevelCollectionCommands>
	(
		"WorldBrowser", // Context name for fast lookup
		NSLOCTEXT("Contexts", "WorldBrowser", "World Browser"), // Localized context name for displaying
		NAME_None, // Parent
		FAppStyle::GetAppStyleSetName() // Icon Style Set
	)
	{
	}
	
	/** Initialize the commands */
	virtual void RegisterCommands() override
	{
		UI_COMMAND( RefreshBrowser,	"Refresh",	"Refreshes opened world", EUserInterfaceActionType::Button, FInputChord(EKeys::F5) );

		//invalid selected level
		UI_COMMAND( FixUpInvalidReference, "Replace Selected Level","Removes the broken level and replaces it with the level browsed to", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( RemoveInvalidReference, "Remove Selected Level", "Removes the reference to the missing level from the map", EUserInterfaceActionType::Button, FInputChord() );
		
		//levels
		UI_COMMAND( World_MakeLevelCurrent, "Make Current", "Make this Level the Current Level", EUserInterfaceActionType::Button, FInputChord( EKeys::Enter ) );
		UI_COMMAND( World_FindInContentBrowser, "Find in Content Browser", "Find the selected levels in the Content Browser", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( World_LoadLevel, "Load", "Load selected level into world", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( World_UnloadLevel, "Unload", "Unload selected level from world", EUserInterfaceActionType::Button, FInputChord(EKeys::Platform_Delete) );
		UI_COMMAND( World_SaveSelectedLevels, "Save", "Saves selected levels", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( World_SaveSelectedLevelAs, "Save As...", "Save the selected level as...", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( World_MigrateSelectedLevels, "Migrate...", "Copies the selected levels and all their dependencies to a different game", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( World_MergeSelectedLevels, "Merge...", "Merges the selected levels into a new level, removing the original levels from the persistent", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( World_CreateNewLevel, "Create New...", "Creates a new level, or choose a level template to start from.", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( World_AddExistingLevel, "Add Existing...", "Adds an existing level", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( World_AddSelectedActorsToNewLevel, "Create New with Selected Actors...", "Adds the actors currently selected in the active viewport to a new Level", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( World_RemoveSelectedLevels, "Remove Selected", "Removes selected levels from the base streaming level", EUserInterfaceActionType::Button, FInputChord() );

		UI_COMMAND( MoveWorldOrigin, "Move World Origin to Level Position", "Moves world origin to level position", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( ResetWorldOrigin, "Reset World Origin", "Moves world origin to zero", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( ResetLevelOrigin, "Reset Level Position", "Moves level to zero", EUserInterfaceActionType::Button, FInputChord() );
		
		//level selection
		UI_COMMAND( SelectAllLevels, "Select All Levels", "Selects all levels", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( DeselectAllLevels, "De-select All Levels", "De-selects all levels", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( InvertLevelSelection, "Invert Level Selection", "Inverts level selection", EUserInterfaceActionType::Button, FInputChord() );

		// Source Control
		UI_COMMAND( SCCCheckOut, "Check Out", "Checks out the selected asset from revision control.", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( SCCCheckIn, "Check In", "Checks in the selected asset to revision control.", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( SCCOpenForAdd, "Mark For Add", "Adds the selected asset to revision control.", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( SCCHistory, "History", "Displays the history of the selected asset in revision control.", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( SCCRefresh, "Refresh", "Updates the revision control status of the asset.", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( SCCDiffAgainstDepot, "Diff Against Depot", "Look at differences between your version of the asset and that in revision control.", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( SCCConnect, "Connect To Revision Control", "Connect to a revision control system for tracking changes to your content and levels.", EUserInterfaceActionType::Button, FInputChord() );

		// set new streaming method
		UI_COMMAND( SetAddStreamingMethod_Blueprint, "Set Blueprint Streaming Method", "Sets the streaming method for new or added levels to Blueprint streaming", EUserInterfaceActionType::RadioButton, FInputChord() );
		UI_COMMAND( SetAddStreamingMethod_AlwaysLoaded, "Set Streaming to Always Loaded", "Sets the streaming method new or added selected levels to be always loaded", EUserInterfaceActionType::RadioButton, FInputChord() );

		// change streaming method
		UI_COMMAND( SetStreamingMethod_Blueprint, "Change Blueprint Streaming Method", "Changes the streaming method for the selected levels to Blueprint streaming", EUserInterfaceActionType::Check, FInputChord() );
		UI_COMMAND( SetStreamingMethod_AlwaysLoaded, "Change Streaming to Always Loaded", "Changes the streaming method for the selected levels to be always loaded", EUserInterfaceActionType::Check, FInputChord() );

		// change lighting scenario
		UI_COMMAND( SetLightingScenario_Enabled, "Make level a Lighting Scenario", "Changes the level to be a Lighting Scenario.  Lighting is built separately for each Lighting Scenario, with all other Scenarios hidden.", EUserInterfaceActionType::Check, FInputChord() );
		UI_COMMAND( SetLightingScenario_Disabled, "Make level not a Lighting Scenario", "Changes the level to not be a Lighting Scenario", EUserInterfaceActionType::Check, FInputChord() );

		//layers
		UI_COMMAND( AssignLevelToLayer, "Assign to layer", "Assign selected levels to current layer", EUserInterfaceActionType::Button, FInputChord() );

		//actors
		UI_COMMAND( AddsActors, "Select Actors", "Adds the Actors in the selected Levels from the viewport's existing selection", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( RemovesActors, "Deselect Actors", "Removes the Actors in the selected Levels from the viewport's existing selection", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND (ConvertLevelToExternalActors, "Convert Level to External Actors", "Enable selected levels to use external actors and convert their actors to external packaging", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( ConvertLevelToInternalActors, "Convert Level to Internal Actors", "Disable selected levels to use external actors and convert their actors to internal packaging", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( MoveActorsToSelected, "Move Selected Actors to Level", "Moves the selected actors to this level", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( MoveFoliageToSelected, "Move Selected Foliage to Level", "Moves the selected foliage instances to this level. Keeps cross-level references to original bases", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( SelectStreamingVolumes, "Select Streaming Volumes", "Selects the streaming volumes associated with the selected levels", EUserInterfaceActionType::Button, FInputChord() );

		//editor visibility
		UI_COMMAND( World_ShowInEditorSelectedLevels, "Show Selected", "Toggles selected levels to a visible state in the viewports for editor worlds", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( World_HideInEditorSelectedLevels, "Hide Selected", "Toggles selected levels to an invisible state in the viewports for editor worlds", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( World_ShowInEditorOnlySelectedLevels, "Show Only Selected", "Toggles the selected levels to a visible state for editor worlds; toggles all other levels to an invisible state", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( World_ShowInEditorAllButSelectedLevels, "Show All But Selected", "Toggles the selected levels to an invisible state for editor worlds; toggles all other levels to a visible state", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND( World_ShowInEditorAllLevels, "Show All", "Toggles all levels to a visible state in the viewports for editor worlds", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND( World_HideInEditorAllLevels, "Hide All", "Hides all levels to an invisible state in the viewports for editor worlds", EUserInterfaceActionType::Button, FInputChord() );

		//game visibility
		UI_COMMAND( World_ShowInGameSelectedLevels, "Show Selected", "Toggles selected levels to a visible state in the viewports for game worlds", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( World_HideInGameSelectedLevels, "Hide Selected", "Toggles selected levels to an invisible state in the viewports for game worlds", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( World_ShowInGameOnlySelectedLevels, "Show Only Selected", "Toggles the selected levels to a visible state for game worlds; toggles all other levels to an invisible state", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( World_ShowInGameAllButSelectedLevels, "Show All But Selected", "Toggles the selected levels to an invisible state for game worlds; toggles all other levels to a visible state", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND( World_ShowInGameAllLevels, "Show All", "Toggles all levels to a visible state in the viewports for game worlds", EUserInterfaceActionType::Button, FInputChord());
		UI_COMMAND( World_HideInGameAllLevels, "Hide All", "Hides all levels to an invisible state in the viewports for game worlds", EUserInterfaceActionType::Button, FInputChord() );

		//lock
		UI_COMMAND( World_LockSelectedLevels, "Lock Selected", "Locks selected levels", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( World_UnlockSelectedLevels, "Unlock Selected", "Unlocks selected levels", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( World_LockOnlySelectedLevels, "Lock Only Selected", "Toggles the selected levels to a locked state; toggles all other levels to an unlocked state", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( World_LockAllButSelectedLevels, "Lock All But Selected", "Toggles the selected levels to an unlocked state; toggles all other levels to a locked state", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( World_LockAllLevels, "Lock All", "Locks all levels", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( World_UnlockAllLevels, "Unlock All", "Unlocks all levels", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( World_LockReadOnlyLevels, "Lock Read-Only Levels", "Locks all read-only levels", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( World_UnlockReadOnlyLevels, "Unlock Read-Only Levels", "Unlocks all read-only levels", EUserInterfaceActionType::Button, FInputChord() );
		
		//view
		UI_COMMAND( FitToSelection, "Fit to Selection", "Fits view to selected levels", EUserInterfaceActionType::Button, FInputChord(EKeys::F) );
		UI_COMMAND( ExpandSelectedItems, "Expand Selected", "Expands all children of a selected items", EUserInterfaceActionType::Button, FInputChord() );
		
		// Parent->child
		UI_COMMAND( ClearParentLink, "Clear Parent Link", "Clears parent link for selected items", EUserInterfaceActionType::Button, FInputChord() );

		UI_COMMAND( MoveLevelLeft,	"Move Level Left",		"Moves level to the left by 1 unit", EUserInterfaceActionType::Button, FInputChord(EKeys::Left) );
		UI_COMMAND( MoveLevelRight, "Move Level Right",		"Moves level to the right by 1 unit", EUserInterfaceActionType::Button, FInputChord(EKeys::Right) );
		UI_COMMAND( MoveLevelUp,	"Move Level Up",		"Moves level up by 1 unit", EUserInterfaceActionType::Button, FInputChord(EKeys::Up) );
		UI_COMMAND( MoveLevelDown,	"Move Level Down",		"Moves level down by 1 unit", EUserInterfaceActionType::Button, FInputChord(EKeys::Down) );
		
		// Landscape operations
		UI_COMMAND( ImportTiledLandscape, "Import Tiled Landscape...", "Imports landscape from a tiled heightmap files (<name>X<n>_Y<n>.png)", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( AddLandscapeLevelXNegative,	"-X",	"Add a new adjacent level with landscape proxy", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( AddLandscapeLevelXPositive,	"+X",	"Add a new adjacent level with landscape proxy", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( AddLandscapeLevelYNegative,	"-Y",	"Add a new adjacent level with landscape proxy", EUserInterfaceActionType::Button, FInputChord() );
		UI_COMMAND( AddLandscapeLevelYPositive,	"+Y",	"Add a new adjacent level with landscape proxy", EUserInterfaceActionType::Button, FInputChord() );
				
		UI_COMMAND( LockTilesLocation, "Lock tiles location", "When enabled all tiles location will be locked, content inside tiles can still be edited", EUserInterfaceActionType::ToggleButton, FInputChord());
	}

public:
	/** refreshes all world cached data */
	TSharedPtr< FUICommandInfo > RefreshBrowser;


	/** Replaces selected invalid level with an already existing one, prompts for path */
	TSharedPtr< FUICommandInfo > FixUpInvalidReference;

	/** Removes the ULevelStreaming reference to the selected invalid levels */
	TSharedPtr< FUICommandInfo > RemoveInvalidReference;
	

	/** makes the selected level the current level */
	TSharedPtr< FUICommandInfo > World_MakeLevelCurrent;

	/** finds the selected levels in the content browser */
	TSharedPtr< FUICommandInfo> World_FindInContentBrowser;

	/** Load level to the world */
	TSharedPtr< FUICommandInfo > World_LoadLevel;

	/** Unload level from the world */
	TSharedPtr< FUICommandInfo > World_UnloadLevel;

	/** Save selected levels; prompts for checkout if necessary */
	TSharedPtr< FUICommandInfo > World_SaveSelectedLevels;

	/** Save selected level under new name; prompts for checkout if necessary */
	TSharedPtr< FUICommandInfo > World_SaveSelectedLevelAs;

	/** Migrate selected levels; copies levels and all their dependencies to another game */
	TSharedPtr< FUICommandInfo > World_MigrateSelectedLevels;

	/** Merges the selected levels into a new level; prompts for save path; removes the levels that were merged */
	TSharedPtr< FUICommandInfo > World_MergeSelectedLevels;

	/** Creates a new level; prompts for save path */
	TSharedPtr< FUICommandInfo > World_CreateNewLevel;

	/** Adds an existing streaming level to the persistent; prompts for path */
	TSharedPtr< FUICommandInfo > World_AddExistingLevel;

	/** Creates a new empty level and moves the selected actors to it; prompts for save path */
	TSharedPtr< FUICommandInfo > World_AddSelectedActorsToNewLevel;

	/** Removes the selected levels from the disk */
	TSharedPtr< FUICommandInfo > World_RemoveSelectedLevels;

	
	/** Move world origin to selected level */
	TSharedPtr< FUICommandInfo > MoveWorldOrigin;
		
	/** Move world origin to zero (reset) */
	TSharedPtr< FUICommandInfo > ResetWorldOrigin;
	
	/** Move level origin to zero (reset) */
	TSharedPtr< FUICommandInfo > ResetLevelOrigin;

	
	/** Selects all levels within the level browser */
	TSharedPtr< FUICommandInfo > SelectAllLevels;

	/** Deselects all levels within the level browser */
	TSharedPtr< FUICommandInfo > DeselectAllLevels;

	/** Inverts the level selection within the level browser */
	TSharedPtr< FUICommandInfo > InvertLevelSelection;


	/** Check-Out selected levels from the SCC */
	TSharedPtr< FUICommandInfo > SCCCheckOut;

	/** Check-In selected levels from the SCC */
	TSharedPtr< FUICommandInfo > SCCCheckIn;

	/** Add selected levels to the SCC */
	TSharedPtr< FUICommandInfo > SCCOpenForAdd;

	/** Open a window to display the SCC history of the selected levels */
	TSharedPtr< FUICommandInfo > SCCHistory;

	/** Refresh the status of selected levels in SCC */
	TSharedPtr< FUICommandInfo > SCCRefresh;

	/** Diff selected levels against the version in the SCC depot */
	TSharedPtr< FUICommandInfo > SCCDiffAgainstDepot;

	/** Enable source control features */
	TSharedPtr< FUICommandInfo > SCCConnect;


	/** Sets the streaming method for new or added levels to Blueprint streaming */
	TSharedPtr< FUICommandInfo > SetAddStreamingMethod_Blueprint;

	/** Sets the streaming method for new or added levels to be always loaded */
	TSharedPtr< FUICommandInfo > SetAddStreamingMethod_AlwaysLoaded;


	/** Changes the streaming method for the selected levels to Blueprint streaming */
	TSharedPtr< FUICommandInfo > SetStreamingMethod_Blueprint;

	/** Changes the streaming method for the selected levels to be always loaded */
	TSharedPtr< FUICommandInfo > SetStreamingMethod_AlwaysLoaded;

	TSharedPtr< FUICommandInfo > SetLightingScenario_Enabled;
	TSharedPtr< FUICommandInfo > SetLightingScenario_Disabled;

	/** Assign selected levels to current layer */
	TSharedPtr< FUICommandInfo > AssignLevelToLayer;


	/** Selects the actors in the selected Levels */
	TSharedPtr< FUICommandInfo > AddsActors;

	/** Deselects the actors in the selected Levels */
	TSharedPtr< FUICommandInfo > RemovesActors;
	
	/** Activate `Use External Actors` on the level and convert actors to external packaging. */
	TSharedPtr< FUICommandInfo > ConvertLevelToExternalActors;

	/** Disable `Use External Actors` on the level and convert actors to internal packaging. */
	TSharedPtr< FUICommandInfo > ConvertLevelToInternalActors;

	/** Moves the selected actors to the selected level */
	TSharedPtr< FUICommandInfo > MoveActorsToSelected;

	/** Moves the selected foliage instances to the selected level */
	TSharedPtr< FUICommandInfo > MoveFoliageToSelected;
	
	/** Selects the streaming volumes associated with the selected levels */
	TSharedPtr< FUICommandInfo > SelectStreamingVolumes;
	
	/** Makes selected Levels visible */
	TSharedPtr< FUICommandInfo > World_ShowInEditorSelectedLevels;

	/** Makes selected Levels invisible */
	TSharedPtr< FUICommandInfo > World_HideInEditorSelectedLevels;

	/** Makes selected Levels visible; makes all others invisible */
	TSharedPtr< FUICommandInfo > World_ShowInEditorOnlySelectedLevels;

	/** Makes selected Levels invisible; makes all others visible */
	TSharedPtr< FUICommandInfo > World_ShowInEditorAllButSelectedLevels;

	/** Makes all Levels visible */
	TSharedPtr< FUICommandInfo > World_ShowInEditorAllLevels;

	/** Makes all Levels invisible */
	TSharedPtr< FUICommandInfo > World_HideInEditorAllLevels;

	
	/** Makes selected Levels visible in the editor */
	TSharedPtr< FUICommandInfo > World_ShowInGameSelectedLevels;
	
	/** Makes selected Levels invisible in the editor */
	TSharedPtr< FUICommandInfo > World_HideInGameSelectedLevels;
	
	/** Makes selected Levels visible; makes all others invisible in the editor */
	TSharedPtr< FUICommandInfo > World_ShowInGameOnlySelectedLevels;
	
	/** Makes selected Levels invisible; makes all others visible in the editor */
	TSharedPtr< FUICommandInfo > World_ShowInGameAllButSelectedLevels;
	
	/** Makes all Levels visible in the editor */
	TSharedPtr< FUICommandInfo > World_ShowInGameAllLevels;
	
	/** Makes all Levels invisible in the editor */
	TSharedPtr< FUICommandInfo > World_HideInGameAllLevels;
	
	
	/** Locks selected levels */
	TSharedPtr< FUICommandInfo > World_LockSelectedLevels;

	/** Unlocks selected levels */
	TSharedPtr< FUICommandInfo > World_UnlockSelectedLevels;

	/** Locks all levels */
	TSharedPtr< FUICommandInfo > World_LockAllLevels;

	/** Unlocks selected levels */
	TSharedPtr< FUICommandInfo > World_UnlockAllLevels;

	/** Makes selected Levels locked; makes all others unlocked */
	TSharedPtr< FUICommandInfo > World_LockOnlySelectedLevels;

	/** Makes selected Levels unlocked; makes all others locked */
	TSharedPtr< FUICommandInfo > World_LockAllButSelectedLevels;

	/** Locks all read-only levels */
	TSharedPtr< FUICommandInfo > World_LockReadOnlyLevels;

	/** Unlocks all read-only levels */
	TSharedPtr< FUICommandInfo > World_UnlockReadOnlyLevels;


	/** Fits view to selected levels */
	TSharedPtr< FUICommandInfo > FitToSelection;

	/** Expand all descendants in selected tree items */
	TSharedPtr< FUICommandInfo > ExpandSelectedItems;


	/** Clear link to parents for selected levels */
	TSharedPtr< FUICommandInfo > ClearParentLink;
	

	// Move levels 
	TSharedPtr< FUICommandInfo > MoveLevelLeft;
	TSharedPtr< FUICommandInfo > MoveLevelRight;
	TSharedPtr< FUICommandInfo > MoveLevelUp;
	TSharedPtr< FUICommandInfo > MoveLevelDown;

	TSharedPtr< FUICommandInfo > LockTilesLocation;
	
	// Landscape operations
	TSharedPtr< FUICommandInfo > ImportTiledLandscape;
	
	TSharedPtr< FUICommandInfo > AddLandscapeLevelXNegative;
	TSharedPtr< FUICommandInfo > AddLandscapeLevelXPositive;
	TSharedPtr< FUICommandInfo > AddLandscapeLevelYNegative;
	TSharedPtr< FUICommandInfo > AddLandscapeLevelYPositive;
};

#undef LOCTEXT_NAMESPACE
