// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelCollectionModel.h"
#include "Algo/AnyOf.h"
#include "Misc/PackageName.h"
#include "AssetRegistry/AssetData.h"
#include "Misc/MessageDialog.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FeedbackContext.h"
#include "Modules/ModuleManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ISourceControlOperation.h"
#include "SourceControlOperations.h"
#include "ISourceControlModule.h"
#include "SourceControlHelpers.h"
#include "Settings/EditorLoadingSavingSettings.h"
#include "Engine/Selection.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "FileHelpers.h"
#include "ScopedTransaction.h"
#include "EditorLevelUtils.h"
#include "LevelCollectionCommands.h"
#include "SourceControlWindows.h"
#include "IAssetTools.h"
#include "IAssetTypeActions.h"
#include "AssetToolsModule.h"
#include "DiffUtils.h"
#include "EditorSupportDelegates.h"
#include "GameFramework/WorldSettings.h"

#include "ShaderCompiler.h"
#include "FoliageEditModule.h"
#include "InstancedFoliageActor.h"
#include "FoliageEditUtility.h"
#include "Engine/WorldComposition.h"
#include "Misc/ScopeExit.h"
#include "LevelUtils.h"
#include "WorldTreeItemTypes.h"

#define LOCTEXT_NAMESPACE "WorldBrowser"

FLevelCollectionModel::FLevelCollectionModel()
	: bRequestedUpdateAllLevels(false)
	, bRequestedRedrawAllLevels(false)
	, bRequestedUpdateActorsCount(false)
	, CommandList(MakeShareable(new FUICommandList))
	, Filters(MakeShareable(new LevelFilterCollection))
	, WorldSize(FIntPoint::ZeroValue)
	, bDisplayPaths(false)
	, bCanExecuteSCCCheckOut(false)
	, bCanExecuteSCCOpenForAdd(false)
	, bCanExecuteSCCCheckIn(false)
	, bCanExecuteSCC(false)
	, bSelectionHasChanged(true)
	, bUpdatingLevelsSelection(false)
{
	FEditorDelegates::RefreshLevelBrowser.AddRaw(this, &FLevelCollectionModel::PopulateLevelsList);
}

FLevelCollectionModel::~FLevelCollectionModel()
{
	SaveSettings();
	
	Filters->OnChanged().RemoveAll(this);
	FWorldDelegates::LevelAddedToWorld.RemoveAll(this);
	FWorldDelegates::LevelRemovedFromWorld.RemoveAll(this);
	FEditorSupportDelegates::RedrawAllViewports.RemoveAll(this);
	GEditor->OnLevelActorAdded().RemoveAll(this);
	GEditor->OnLevelActorDeleted().RemoveAll(this);
	if (CurrentWorld.IsValid())
	{
		CurrentWorld->OnSelectedLevelsChanged().RemoveAll(this);
	}

	FEditorDelegates::RefreshLevelBrowser.RemoveAll(this);
}

void FLevelCollectionModel::Initialize(UWorld* InWorld)
{
	LoadSettings();
	
	CurrentWorld = InWorld;

	Filters->OnChanged().AddSP(this, &FLevelCollectionModel::OnFilterChanged);
	FWorldDelegates::LevelAddedToWorld.AddSP(this, &FLevelCollectionModel::OnLevelAddedToWorld);
	FWorldDelegates::LevelRemovedFromWorld.AddSP(this, &FLevelCollectionModel::OnLevelRemovedFromWorld);
	FEditorSupportDelegates::RedrawAllViewports.AddSP(this, &FLevelCollectionModel::OnRedrawAllViewports);
	GEditor->OnLevelActorAdded().AddSP( this, &FLevelCollectionModel::OnLevelActorAdded);
	GEditor->OnLevelActorDeleted().AddSP( this, &FLevelCollectionModel::OnLevelActorDeleted);
	USelection::SelectionChangedEvent.AddSP(this, &FLevelCollectionModel::OnActorSelectionChanged);
	SelectionChanged.AddSP(this, &FLevelCollectionModel::OnActorOrLevelSelectionChanged);
	CurrentWorld->OnSelectedLevelsChanged().AddSP(this, &FLevelCollectionModel::OnLevelsSelectionChangedOutside);

	PopulateLevelsList();
}

void FLevelCollectionModel::BindCommands()
{
	const FLevelCollectionCommands& Commands = FLevelCollectionCommands::Get();
	FUICommandList& ActionList = *CommandList;

	ActionList.MapAction(Commands.RefreshBrowser,
		FExecuteAction::CreateSP(this, &FLevelCollectionModel::RefreshBrowser_Executed));

	ActionList.MapAction(Commands.ExpandSelectedItems,
		FExecuteAction::CreateSP(this, &FLevelCollectionModel::ExpandSelectedItems_Executed),
		FCanExecuteAction::CreateSP(this, &FLevelCollectionModel::AreAnyLevelsSelected));

	ActionList.MapAction(Commands.World_MakeLevelCurrent,
		FExecuteAction::CreateSP(this, &FLevelCollectionModel::MakeLevelCurrent_Executed),
		FCanExecuteAction::CreateSP(this, &FLevelCollectionModel::IsOneLevelSelected));
	
	ActionList.MapAction(Commands.World_FindInContentBrowser,
		FExecuteAction::CreateSP(this, &FLevelCollectionModel::FindInContentBrowser_Executed),
		FCanExecuteAction::CreateSP(this, &FLevelCollectionModel::IsValidFindInContentBrowser));
	
	ActionList.MapAction(Commands.MoveActorsToSelected,
		FExecuteAction::CreateSP(this, &FLevelCollectionModel::MoveActorsToSelected_Executed),
		FCanExecuteAction::CreateSP(this, &FLevelCollectionModel::IsValidMoveActorsToLevel));

	ActionList.MapAction(Commands.MoveFoliageToSelected,
		FExecuteAction::CreateSP(this, &FLevelCollectionModel::MoveFoliageToSelected_Executed),
		FCanExecuteAction::CreateSP(this, &FLevelCollectionModel::IsValidMoveFoliageToLevel));
		
	ActionList.MapAction(Commands.World_SaveSelectedLevels,
		FExecuteAction::CreateSP(this, &FLevelCollectionModel::SaveSelectedLevels_Executed),
		FCanExecuteAction::CreateSP(this, &FLevelCollectionModel::AreAnySelectedLevelsDirty));

	ActionList.MapAction(Commands.World_SaveSelectedLevelAs,
		FExecuteAction::CreateSP(this, &FLevelCollectionModel::SaveSelectedLevelAs_Executed),
		FCanExecuteAction::CreateSP(this, &FLevelCollectionModel::IsSelectedLevelEditable));
		
	ActionList.MapAction(Commands.World_LoadLevel,
		FExecuteAction::CreateSP(this, &FLevelCollectionModel::LoadSelectedLevels_Executed),
		FCanExecuteAction::CreateSP(this, &FLevelCollectionModel::AreAnySelectedLevelsUnloaded));

	ActionList.MapAction(Commands.World_UnloadLevel,
		FExecuteAction::CreateSP(this, &FLevelCollectionModel::UnloadSelectedLevels_Executed),
		FCanExecuteAction::CreateSP(this, &FLevelCollectionModel::AreAllSelectedLevelsUserManaged));

	ActionList.MapAction( Commands.World_MigrateSelectedLevels,
		FExecuteAction::CreateSP( this, &FLevelCollectionModel::MigrateSelectedLevels_Executed),
		FCanExecuteAction::CreateSP( this, &FLevelCollectionModel::AreAllSelectedLevelsEditable));

	//actors
	ActionList.MapAction(Commands.AddsActors,
		FExecuteAction::CreateSP(this, &FLevelCollectionModel::SelectActors_Executed),
		FCanExecuteAction::CreateSP(this, &FLevelCollectionModel::AreAnySelectedLevelsEditable));
	
	ActionList.MapAction(Commands.RemovesActors,
		FExecuteAction::CreateSP(this, &FLevelCollectionModel::DeselectActors_Executed),
		FCanExecuteAction::CreateSP(this, &FLevelCollectionModel::AreAnySelectedLevelsEditable));

	ActionList.MapAction(Commands.ConvertLevelToExternalActors,
		FExecuteAction::CreateSP(this, &FLevelCollectionModel::ConvertLevelToExternalActors_Executed, true),
		FCanExecuteAction::CreateSP(this, &FLevelCollectionModel::CanConvertAnyLevelToExternalActors, true));

	ActionList.MapAction(Commands.ConvertLevelToInternalActors,
		FExecuteAction::CreateSP(this, &FLevelCollectionModel::ConvertLevelToExternalActors_Executed, false),
		FCanExecuteAction::CreateSP(this, &FLevelCollectionModel::CanConvertAnyLevelToExternalActors, false));

	//editor visibility
	ActionList.MapAction( Commands.World_ShowInEditorSelectedLevels,
		FExecuteAction::CreateSP( this, &FLevelCollectionModel::ShowInEditorSelectedLevels_Executed  ),
		FCanExecuteAction::CreateSP( this, &FLevelCollectionModel::AreAnySelectedLevelsLoaded ) );
	ActionList.MapAction( Commands.World_HideInEditorSelectedLevels,
		FExecuteAction::CreateSP( this, &FLevelCollectionModel::HideInEditorSelectedLevels_Executed  ),
		FCanExecuteAction::CreateSP( this, &FLevelCollectionModel::AreAnySelectedLevelsLoaded ) );
	ActionList.MapAction( Commands.World_ShowInEditorOnlySelectedLevels,
		FExecuteAction::CreateSP( this, &FLevelCollectionModel::ShowInEditorOnlySelectedLevels_Executed  ),
		FCanExecuteAction::CreateSP( this, &FLevelCollectionModel::AreAnySelectedLevelsLoaded ) );
	ActionList.MapAction(Commands.World_ShowInEditorAllButSelectedLevels,
		FExecuteAction::CreateSP(this, &FLevelCollectionModel::ShowInEditorAllButSelectedLevels_Executed),
		FCanExecuteAction::CreateSP(this, &FLevelCollectionModel::AreAnySelectedLevelsLoaded));
	ActionList.MapAction(Commands.World_ShowInEditorAllLevels,
		FExecuteAction::CreateSP(this, &FLevelCollectionModel::ShowInEditorAllLevels_Executed));
	ActionList.MapAction(Commands.World_HideInEditorAllLevels,
		FExecuteAction::CreateSP(this, &FLevelCollectionModel::HideInEditorAllLevels_Executed));
	
	//game visibility
	ActionList.MapAction( Commands.World_ShowInGameSelectedLevels,
		FExecuteAction::CreateSP( this, &FLevelCollectionModel::ShowInGameSelectedLevels_Executed  ),
		FCanExecuteAction::CreateSP( this, &FLevelCollectionModel::CanExecuteGameVisibilityCommandsForSelectedLevels ) );
	ActionList.MapAction( Commands.World_HideInGameSelectedLevels,
		FExecuteAction::CreateSP( this, &FLevelCollectionModel::HideInGameSelectedLevels_Executed  ),
		FCanExecuteAction::CreateSP( this, &FLevelCollectionModel::CanExecuteGameVisibilityCommandsForSelectedLevels ) );
	ActionList.MapAction( Commands.World_ShowInGameOnlySelectedLevels,
		FExecuteAction::CreateSP( this, &FLevelCollectionModel::ShowInGameOnlySelectedLevels_Executed  ),
		FCanExecuteAction::CreateSP( this, &FLevelCollectionModel::CanExecuteGameVisibilityCommandsForSelectedLevels ) );
	ActionList.MapAction(Commands.World_ShowInGameAllButSelectedLevels,
		FExecuteAction::CreateSP(this, &FLevelCollectionModel::ShowInGameAllButSelectedLevels_Executed),
		FCanExecuteAction::CreateSP(this, &FLevelCollectionModel::CanExecuteGameVisibilityCommandsForSelectedLevels));
	ActionList.MapAction(Commands.World_ShowInGameAllLevels,
		FExecuteAction::CreateSP(this, &FLevelCollectionModel::ShowInGameAllLevels_Executed),
		FCanExecuteAction::CreateSP(this, &FLevelCollectionModel::CanExecuteGameVisibilityCommands));
	ActionList.MapAction(Commands.World_HideInGameAllLevels,
		FExecuteAction::CreateSP(this, &FLevelCollectionModel::HideInGameAllLevels_Executed),
		FCanExecuteAction::CreateSP(this, &FLevelCollectionModel::CanExecuteGameVisibilityCommands));
		
	//lock
	ActionList.MapAction( Commands.World_LockSelectedLevels,
		FExecuteAction::CreateSP( this, &FLevelCollectionModel::LockSelectedLevels_Executed  ) );
	
	ActionList.MapAction( Commands.World_UnlockSelectedLevels,
		FExecuteAction::CreateSP( this, &FLevelCollectionModel::UnlockSelectedLevels_Executed  ) );

	ActionList.MapAction(Commands.World_LockOnlySelectedLevels,
		FExecuteAction::CreateSP(this, &FLevelCollectionModel::LockOnlySelectedLevels_Executed));

	ActionList.MapAction(Commands.World_LockAllButSelectedLevels,
		FExecuteAction::CreateSP(this, &FLevelCollectionModel::LockAllButSelectedLevels_Executed));
	
	ActionList.MapAction( Commands.World_LockAllLevels,
		FExecuteAction::CreateSP( this, &FLevelCollectionModel::LockAllLevels_Executed  ) );
	
	ActionList.MapAction( Commands.World_UnlockAllLevels,
		FExecuteAction::CreateSP( this, &FLevelCollectionModel::UnlockAllLevels_Executed  ) );

	ActionList.MapAction( Commands.World_LockReadOnlyLevels,
		FExecuteAction::CreateSP( this, &FLevelCollectionModel::ToggleReadOnlyLevels_Executed  ) );

	ActionList.MapAction( Commands.World_UnlockReadOnlyLevels,
		FExecuteAction::CreateSP( this, &FLevelCollectionModel::ToggleReadOnlyLevels_Executed  ) );
	
	
	//level selection
	ActionList.MapAction( Commands.SelectAllLevels,
		FExecuteAction::CreateSP( this, &FLevelCollectionModel::SelectAllLevels_Executed  ) );
	
	ActionList.MapAction( Commands.DeselectAllLevels,
		FExecuteAction::CreateSP( this, &FLevelCollectionModel::DeselectAllLevels_Executed  ) );

	ActionList.MapAction( Commands.InvertLevelSelection,
		FExecuteAction::CreateSP( this, &FLevelCollectionModel::InvertSelection_Executed  ) );
	
	//source control
	ActionList.MapAction( Commands.SCCCheckOut,
		FExecuteAction::CreateSP( this, &FLevelCollectionModel::OnSCCCheckOut  ) );

	ActionList.MapAction( Commands.SCCCheckIn,
		FExecuteAction::CreateSP( this, &FLevelCollectionModel::OnSCCCheckIn  ) );

	ActionList.MapAction( Commands.SCCOpenForAdd,
		FExecuteAction::CreateSP( this, &FLevelCollectionModel::OnSCCOpenForAdd  ) );

	ActionList.MapAction( Commands.SCCHistory,
		FExecuteAction::CreateSP( this, &FLevelCollectionModel::OnSCCHistory  ) );

	ActionList.MapAction( Commands.SCCRefresh,
		FExecuteAction::CreateSP( this, &FLevelCollectionModel::OnSCCRefresh  ) );

	ActionList.MapAction( Commands.SCCDiffAgainstDepot,
		FExecuteAction::CreateSP( this, &FLevelCollectionModel::OnSCCDiffAgainstDepot  ) );

	ActionList.MapAction( Commands.SCCConnect,
		FExecuteAction::CreateSP( this, &FLevelCollectionModel::OnSCCConnect  ) );
	
}

void FLevelCollectionModel::PopulateLevelsList()
{
	RootLevelsList.Empty();
	AllLevelsList.Empty();
	FilteredLevelsList.Empty();
	SelectedLevelsList.Empty();
	AllLevelsMap.Empty();

	OnLevelsCollectionChanged();
}

void FLevelCollectionModel::PopulateFilteredLevelsList()
{
	FilteredLevelsList.Empty();

	// Filter out our flat list
	for (auto& LevelModel : AllLevelsList)
	{
		LevelModel->SetLevelFilteredOutFlag(true);
		if (LevelModel->IsPersistent() || PassesAllFilters(*LevelModel))
		{
			FilteredLevelsList.Add(LevelModel);
			LevelModel->SetLevelFilteredOutFlag(false);
		}
	}

	// Walk through hierarchy and filter it out
	for (auto It = RootLevelsList.CreateIterator(); It; ++It)
	{
		(*It)->OnFilterChanged();
	}
}

void FLevelCollectionModel::Tick( float DeltaTime )
{
	if (GEditor == nullptr)
	{
		// Could it be during hot-reloading?
		return;
	}
	
	if (bRequestedUpdateAllLevels)
	{
		UpdateAllLevels();
	}

	if (bRequestedRedrawAllLevels)
	{		
		RedrawAllLevels();
	}

	if (bRequestedUpdateActorsCount)
	{
		UpdateLevelActorsCount();
	}
	
	if (IsSimulating())
	{
		// Reset simulation status for all levels
		for (TSharedPtr<FLevelModel>& LevelModel : AllLevelsList)
		{
			LevelModel->UpdateSimulationStatus(nullptr);
		}

		// Traverse streaming levels and update simulation status for corresponding level models
		for (ULevelStreaming* StreamingLevel : GetSimulationWorld()->GetStreamingLevels())
		{
			// Rebuild the original NonPrefixedPackageName so we can find it
			const FString PrefixedPackageName = StreamingLevel->GetWorldAssetPackageName();
			const FString NonPrefixedPackageName = FPackageName::GetLongPackagePath(PrefixedPackageName) + "/" 
					+ FPackageName::GetLongPackageAssetName(PrefixedPackageName).RightChop(GetSimulationWorld()->StreamingLevelsPrefix.Len());
								
			TSharedPtr<FLevelModel> LevelModel = FindLevelModel(FName(*NonPrefixedPackageName));
			if (LevelModel.IsValid())
			{
				LevelModel->UpdateSimulationStatus(StreamingLevel);
			}
		}
	}
}

TStatId FLevelCollectionModel::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FLevelCollectionModel, STATGROUP_Tickables);
}

bool FLevelCollectionModel::IsReadOnly() const
{
	// read only in PIE/SIE
	return IsSimulating();
}

bool FLevelCollectionModel::IsSimulating() const
{
	return (GEditor->bIsSimulatingInEditor || GEditor->PlayWorld != NULL);
}

UWorld* FLevelCollectionModel::GetSimulationWorld() const 
{ 
	return GEditor->PlayWorld; 
}

bool FLevelCollectionModel::IsOriginRebasingEnabled() const
{
	UWorld* ThisWorld = GetWorld();
	return ThisWorld && ThisWorld->GetWorldSettings()->bEnableWorldOriginRebasing;
}

FLevelModelList& FLevelCollectionModel::GetRootLevelList()
{ 
	return RootLevelsList;
}

const FLevelModelList& FLevelCollectionModel::GetAllLevels() const
{
	return AllLevelsList;
}

const FLevelModelList& FLevelCollectionModel::GetFilteredLevels() const
{
	return FilteredLevelsList;
}

const FLevelModelList& FLevelCollectionModel::GetSelectedLevels() const
{
	return SelectedLevelsList;
}

void FLevelCollectionModel::AddFilter(const TSharedRef<LevelFilter>& InFilter)
{
	Filters->Add(InFilter);
	OnFilterChanged();
}

void FLevelCollectionModel::RemoveFilter(const TSharedRef<LevelFilter>& InFilter)
{
	Filters->Remove(InFilter);
	OnFilterChanged();
}

bool FLevelCollectionModel::IsFilterActive() const
{
	return (AllLevelsList.Num() != FilteredLevelsList.Num());
}

void FLevelCollectionModel::SetSelectedLevels(const FLevelModelList& InList)
{
	// Clear selection flag from currently selected levels
	for (auto LevelModel : SelectedLevelsList)
	{
		LevelModel->SetLevelSelectionFlag(false);
	}
	
	SelectedLevelsList.Empty(); 
	
	// Set selection flag to selected levels	
	for (auto& LevelModel : InList)
	{
		if (LevelModel.IsValid() && PassesAllFilters(*LevelModel))
		{
			LevelModel->SetLevelSelectionFlag(true);
			SelectedLevelsList.Add(LevelModel);
		}
	}

	OnLevelsSelectionChanged();
}

void FLevelCollectionModel::SetSelectedLevelsFromWorld()
{
	TArray<TObjectPtr<ULevel>>& SelectedLevelObjects = CurrentWorld->GetSelectedLevels();
	FLevelModelList LevelsToSelect;
	for (ULevel* LevelObject : SelectedLevelObjects)
	{
		TSharedPtr<FLevelModel> LevelModel = FindLevelModel(LevelObject);
		if (LevelModel.IsValid())
		{
			LevelsToSelect.Add(LevelModel);
		}
	}

	SetSelectedLevels(LevelsToSelect);
}

TSharedPtr<FLevelModel> FLevelCollectionModel::FindLevelModel(ULevel* InLevel) const
{
	if (InLevel)
	{
		for (auto It = AllLevelsList.CreateConstIterator(); It; ++It)
		{
			if ((*It)->GetLevelObject() == InLevel)
			{
				return (*It);
			}
		}
	}

	// not found
	return TSharedPtr<FLevelModel>();
}

TSharedPtr<FLevelModel> FLevelCollectionModel::FindLevelModel(const FName& PackageName) const
{
	const TSharedPtr<FLevelModel>* LevelModel = AllLevelsMap.Find(PackageName);
	if (LevelModel != NULL)
	{
		return *LevelModel;
	}
	
	// not found
	return TSharedPtr<FLevelModel>();
}

void FLevelCollectionModel::IterateHierarchy(FLevelModelVisitor& Visitor)
{
	for (auto It = RootLevelsList.CreateIterator(); It; ++It)
	{
		(*It)->Accept(Visitor);
	}
}

void FLevelCollectionModel::HideLevelsInEditor(const FLevelModelList& InLevelList)
{
	if (IsReadOnly())
	{
		return;
	}
	
	// Disable Origin Tracking
	UWorldComposition* WorldComposition = GetWorld()->WorldComposition;
	if (WorldComposition)
	{
		WorldComposition->bTemporarilyDisableOriginTracking = true;
	}
	ON_SCOPE_EXIT
	{
		// Reenable Origin Tracking
		if (WorldComposition)
		{
			WorldComposition->bTemporarilyDisableOriginTracking = false;
		}
	};

	// For efficiency, set visibility of all levels at once
	TArray<FLevelModel*> LevelModels;
	TArray<bool> bVisible;
	for (auto It = InLevelList.CreateConstIterator(); It; ++It)
	{
		LevelModels.Add(It->Get());
		bVisible.Add(false);
	}
	FLevelModel::SetVisibleInEditor(LevelModels, bVisible);

	RequestUpdateAllLevels();
}
	
void FLevelCollectionModel::ShowLevelsInEditor(const FLevelModelList& InLevelList)
{
	if (IsReadOnly())
	{
		return;
	}
	
	// Disable Origin Tracking
	UWorldComposition* WorldComposition = GetWorld()->WorldComposition;
	if (WorldComposition)
	{
		WorldComposition->bTemporarilyDisableOriginTracking = true;
	}
	ON_SCOPE_EXIT
	{
		// Reenable Origin Tracking
		if (WorldComposition)
		{
			WorldComposition->bTemporarilyDisableOriginTracking = false;
		}
	};

	OnPreShowLevels(InLevelList);

	// For efficiency, set visibility of all levels at once
	TArray<FLevelModel*> LevelModels;
	TArray<bool> bVisible;
	for (auto It = InLevelList.CreateConstIterator(); It; ++It)
	{
		LevelModels.Add(It->Get());
		bVisible.Add(true);
	}
	FLevelModel::SetVisibleInEditor(LevelModels, bVisible);

	RequestUpdateAllLevels();
}

void FLevelCollectionModel::ShowInEditorOnlySelectedLevels()
{
	ShowInEditorOnlySelectedLevels_Executed();
}

void FLevelCollectionModel::ShowInEditorAllButSelectedLevels()
{
	ShowInEditorAllButSelectedLevels_Executed();
}

void FLevelCollectionModel::HideLevelsInGame(const FLevelModelList& InLevelList)
{
	// For efficiency, set visibility of all levels at once
	TArray<FLevelModel*> LevelModels;
	TArray<bool> bVisible;
	for (auto It = InLevelList.CreateConstIterator(); It; ++It)
	{
		LevelModels.Add(It->Get());
		bVisible.Add(false);
	}
	FLevelModel::SetVisibleInGame(LevelModels, bVisible);
}

void FLevelCollectionModel::ShowLevelsInGame(const FLevelModelList& InLevelList)
{
	// For efficiency, set visibility of all levels at once
	TArray<FLevelModel*> LevelModels;
	TArray<bool> bVisible;
	for (auto It = InLevelList.CreateConstIterator(); It; ++It)
	{
		LevelModels.Add(It->Get());
		bVisible.Add(true);
	}
	FLevelModel::SetVisibleInGame(LevelModels, bVisible);
}

void FLevelCollectionModel::ShowInGameOnlySelectedLevels()
{
	ShowInGameOnlySelectedLevels_Executed();
}

void FLevelCollectionModel::ShowInGameAllButSelectedLevels()
{
	ShowInGameAllButSelectedLevels_Executed();
}

void FLevelCollectionModel::UnlockLevels(const FLevelModelList& InLevelList)
{
	if (!IsReadOnly())
	{
		const FText UndoTransactionText = (InLevelList.Num() == 1) ?
			LOCTEXT("UnlockLevel", "Unlock Level") :
			LOCTEXT("UnlockMultipleLevels", "Unlock Multiple Levels");

		const FScopedTransaction Transaction(UndoTransactionText);

		for (auto It = InLevelList.CreateConstIterator(); It; ++It)
		{
			(*It)->SetLocked(false);
		}
	}
}
	
void FLevelCollectionModel::LockLevels(const FLevelModelList& InLevelList)
{
	if (!IsReadOnly())
	{
		const FText UndoTransactionText = (InLevelList.Num() == 1) ?
			LOCTEXT("LockLevel", "Lock Level") :
			LOCTEXT("LockMultipleLevels", "Lock Multiple Levels");

		const FScopedTransaction Transaction(UndoTransactionText);

		for (auto It = InLevelList.CreateConstIterator(); It; ++It)
		{
			(*It)->SetLocked(true);
		}
	}
}

void FLevelCollectionModel::LockOnlySelectedLevels()
{
	LockOnlySelectedLevels_Executed();
}

void FLevelCollectionModel::LockAllButSelectedLevels()
{
	LockAllButSelectedLevels_Executed();
}

void FLevelCollectionModel::SaveLevels(const FLevelModelList& InLevelList)
{
	if (IsReadOnly())
	{
		return;
	}

		
	FLevelModelList		LevelModelsToSave;
	TArray<ULevel*>		LevelsToSave;
	for (auto It = InLevelList.CreateConstIterator(); It; ++It)
	{
		if ((*It)->GetLevelObject())
		{
			if (!(*It)->IsVisibleInEditor())
			{
				FMessageDialog::Open( EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "UnableToSaveInvisibleLevels", "Save aborted.  Levels must be made visible before they can be saved.") );
				return;
			}
			else if ((*It)->IsLocked())
			{
				FMessageDialog::Open( EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "UnableToSaveLockedLevels", "Save aborted.  Level must be unlocked before it can be saved.") );
				return;
			}
						
			LevelModelsToSave.Add(*It);
			LevelsToSave.Add((*It)->GetLevelObject());
		}
	}

	TArray< UPackage* > PackagesNotNeedingCheckout;

	// Check dirtiness in case of level using external actors to avoid taking in checkout all actors
	bool bCheckDirty = Algo::AnyOf(LevelsToSave, [](const ULevel* InLevel) -> bool { return (InLevel != nullptr) && InLevel->IsUsingExternalActors(); });

	// Prompt the user to check out the levels from source control before saving
	if (FEditorFileUtils::PromptToCheckoutLevels(bCheckDirty, LevelsToSave, &PackagesNotNeedingCheckout))
	{
		for (auto It = LevelsToSave.CreateIterator(); It; ++It)
		{
			FEditorFileUtils::SaveLevel(*It);
		}

		// Add all files that needs to be marked for add in one command, if any
		if (GEditor)
		{
			GEditor->RunDeferredMarkForAddFiles();
		}
	}
	else if (PackagesNotNeedingCheckout.Num() > 0)
	{
		// The user canceled the checkout dialog but some packages didn't need to be checked out in order to save
		// For each selected level if the package its in didn't need to be saved, save the level!
		for (int32 LevelIdx = 0; LevelIdx < LevelsToSave.Num(); ++LevelIdx)
		{
			ULevel* Level = LevelsToSave[LevelIdx];
			if (PackagesNotNeedingCheckout.Contains(Level->GetOutermost()))
			{
				FEditorFileUtils::SaveLevel(Level);
			}
			else
			{
				//remove it from the list, so that only successfully saved levels are highlighted when saving is complete
				LevelModelsToSave.RemoveAt(LevelIdx);
				LevelsToSave.RemoveAt(LevelIdx);
			}
		}
	}

	// Select tiles that were saved successfully
	SetSelectedLevels(LevelModelsToSave);
}

void FLevelCollectionModel::LoadLevels(const FLevelModelList& InLevelList)
{
	if (IsReadOnly())
	{
		return;
	}

	GWarn->BeginSlowTask(LOCTEXT("LoadWorldTiles", "Loading levels"), true);
	
	OnPreLoadLevels(InLevelList);
	
	int32 LevelIdx = 0;
	for (TSharedPtr<FLevelModel> LevelModel : InLevelList)
	{
		GWarn->StatusUpdate(LevelIdx++, InLevelList.Num(), 
			FText::Format(LOCTEXT("LoadingWorldTiles", "Loading: {0}..." ), FText::FromString(LevelModel->GetLongPackageName().ToString()))
			);

		LevelModel->LoadLevel();
	}

	if (InLevelList.Num() > 0)
	{
		GEditor->ResetTransaction(LOCTEXT("LoadingWorldTilesTransReset", "Loading Levels"));
	}

	GWarn->EndSlowTask();	
}

void FLevelCollectionModel::UnloadLevels(const FLevelModelList& InLevelList)
{
	if (InLevelList.Num() == 0)
	{
		return;
	}

	UWorld* ThisWorld = GetWorld();
	check(ThisWorld != nullptr);

	if(GLevelEditorModeTools().IsModeActive(FBuiltinEditorModes::EM_Landscape))
	{
		GLevelEditorModeTools().ActivateDefaultMode();
	}

	BroadcastPreLevelsUnloaded();

	// Take a copy of the list rather than using a reference to the selected levels list, as this will be modified in the loop below
	const FLevelModelList LevelListCopy = InLevelList;
	for (auto It = LevelListCopy.CreateConstIterator(); It; ++It)
	{
		TSharedPtr<FLevelModel> LevelModel = (*It);
		ULevel* Level = LevelModel->GetLevelObject();

		if (Level != nullptr && !LevelModel->IsPersistent() && LevelModel->IsUserManaged())
		{
			// Unselect all actors before removing the level
			// This avoids crashing in areas that rely on getting a selected actors level. The level will be invalid after its removed.
			for (auto ActorIt = Level->Actors.CreateIterator(); ActorIt; ++ActorIt)
			{
				GEditor->SelectActor((*ActorIt), /*bInSelected=*/ false, /*bSelectEvenIfHidden=*/ false);
			}
			
			// In case we have created temporary streaming level object for this sub-level - remove it before unloading sub-level
			{
				FName LevelPackageName = LevelModel->GetLongPackageName();
				auto Predicate = [&](ULevelStreaming* StreamingLevel) 
				{
					return (StreamingLevel && StreamingLevel->GetWorldAssetPackageFName() == LevelPackageName && StreamingLevel->HasAnyFlags(RF_Transient));
				};
				
				if (ULevelStreaming*const* StreamingLevel = ThisWorld->GetStreamingLevels().FindByPredicate(Predicate))
				{
					(*StreamingLevel)->MarkAsGarbage();
					ThisWorld->RemoveStreamingLevel(*StreamingLevel);
				}
			}

			EditorLevelUtils::RemoveLevelFromWorld(Level);
		}
		else if (ULevelStreaming* StreamingLevel = Cast<ULevelStreaming>(LevelModel->GetNodeObject()))
		{
			StreamingLevel->MarkAsGarbage();
			ThisWorld->RemoveStreamingLevel(StreamingLevel);
		}
	}

	BroadcastPostLevelsUnloaded();

	GEditor->ResetTransaction( LOCTEXT("RemoveLevelTransReset", "Removing Levels from World") );

	// Collect garbage to clear out the destroyed level
	CollectGarbage( GARBAGE_COLLECTION_KEEPFLAGS );

	PopulateLevelsList();
}

void FLevelCollectionModel::TranslateLevels(const FLevelModelList& InLevels, FVector2D InDelta, bool bSnapDelta)
{
}

FVector2D FLevelCollectionModel::SnapTranslationDelta(const FLevelModelList& InLevelList, FVector2D InTranslationDelta, bool bBoundsSnapping, FVector2D::FReal InSnappingValue)
{
	return InTranslationDelta;
}

void FLevelCollectionModel::UpdateTranslationDelta(const FLevelModelList& InLevelList, FVector2D InTranslationDelta, bool bBoundsSnapping, FVector2D::FReal InSnappingValue)
{
	FLevelModelList EditableLevels;
	// Only editable levels could be moved
	for (auto It = InLevelList.CreateConstIterator(); It; ++It)
	{
		if ((*It)->IsEditable())
		{
			EditableLevels.Add(*It);
		}
	}
	
	// Snap	translation delta
	if (InTranslationDelta != FVector2D::ZeroVector)
	{
		InTranslationDelta = SnapTranslationDelta(EditableLevels, InTranslationDelta, bBoundsSnapping, InSnappingValue);
	}
		
	for (auto It = EditableLevels.CreateIterator(); It; ++It)
	{
		(*It)->SetLevelTranslationDelta(InTranslationDelta);
	}
}

void FLevelCollectionModel::AssignParent(const FLevelModelList& InLevels, TSharedPtr<FLevelModel> InParent)
{
	// Attach levels to the new parent
	for (auto It = InLevels.CreateConstIterator(); It; ++It)
	{
		(*It)->AttachTo(InParent);
	}

	OnLevelsHierarchyChanged();
}

void FLevelCollectionModel::AddExistingLevelsFromAssetData(const TArray<FAssetData>& WorldList)
{
	
}

TSharedPtr<WorldHierarchy::FWorldBrowserDragDropOp> FLevelCollectionModel::CreateDragDropOp() const
{
	return MakeShareable( new WorldHierarchy::FWorldBrowserDragDropOp );
}

TSharedPtr<WorldHierarchy::FWorldBrowserDragDropOp> FLevelCollectionModel::CreateDragDropOp(const FLevelModelList& InLevels) const
{
	return TSharedPtr<WorldHierarchy::FWorldBrowserDragDropOp>();
}

bool FLevelCollectionModel::PassesAllFilters(const FLevelModel& Item) const
{
	if (Item.IsPersistent() || Filters->PassesAllFilters(&Item))
	{
		return true;
	}
	
	return false;
}

void FLevelCollectionModel::BuildHierarchyMenu(FMenuBuilder& InMenuBuilder, EBuildHierarchyMenuFlags Flags) const
{
}

void FLevelCollectionModel::CustomizeFileMainMenu(FMenuBuilder& InMenuBuilder) const
{
	const FLevelCollectionCommands& Commands = FLevelCollectionCommands::Get();

	// Cache SCC state
	CacheCanExecuteSourceControlVars();

	InMenuBuilder.AddSubMenu( 
		LOCTEXT("SourceControl", "Revision Control"),
		LOCTEXT("SourceControl_ToolTip", "Revision Control Options"),
		FNewMenuDelegate::CreateSP(const_cast<FLevelCollectionModel*>(this), &FLevelCollectionModel::FillSourceControlSubMenu));
		
	if (AreAnyLevelsSelected())
	{
		InMenuBuilder.AddMenuEntry( Commands.World_SaveSelectedLevels );
		InMenuBuilder.AddMenuEntry( Commands.World_SaveSelectedLevelAs );
		InMenuBuilder.AddMenuEntry( Commands.World_MigrateSelectedLevels );
	}
}

bool FLevelCollectionModel::GetPlayerView(FVector& Location, FRotator& Rotation) const
{
	return false;
}

bool FLevelCollectionModel::GetObserverView(FVector& Location, FRotator& Rotation) const
{
	return false;
}

bool FLevelCollectionModel::CompareLevelsZOrder(TSharedPtr<FLevelModel> InA, TSharedPtr<FLevelModel> InB) const
{
	return false;
}

void FLevelCollectionModel::RegisterDetailsCustomization(FPropertyEditorModule& InPropertyModule, 
																TSharedPtr<IDetailsView> InDetailsView)
{
}

void FLevelCollectionModel::UnregisterDetailsCustomization(FPropertyEditorModule& InPropertyModule,
															   TSharedPtr<IDetailsView> InDetailsView)
{
}

void FLevelCollectionModel::RequestUpdateAllLevels()
{
	bRequestedUpdateAllLevels = true;
}

void FLevelCollectionModel::RequestRedrawAllLevels()
{
	bRequestedRedrawAllLevels = true;
}

void FLevelCollectionModel::UpdateAllLevels()
{
	bRequestedUpdateAllLevels = false;

	for (auto It = AllLevelsList.CreateConstIterator(); It; ++It)
	{
		(*It)->Update();
	}
	
	// Update world size
	FBox WorldBounds = GetLevelsBoundingBox(AllLevelsList, false);
	WorldSize.X = FMath::RoundToInt(WorldBounds.GetSize().X);
	WorldSize.Y = FMath::RoundToInt(WorldBounds.GetSize().Y);
}

void FLevelCollectionModel::RedrawAllLevels()
{
	bRequestedRedrawAllLevels = false;

	for (auto It = AllLevelsList.CreateConstIterator(); It; ++It)
	{
		(*It)->UpdateVisuals();
	}
}

void FLevelCollectionModel::UpdateLevelActorsCount()
{
	for( auto It = AllLevelsList.CreateIterator(); It; ++It )
	{
		(*It)->UpdateLevelActorsCount();
	}

	bRequestedUpdateActorsCount = false;
}

bool FLevelCollectionModel::IsOneLevelSelected() const
{
	return SelectedLevelsList.Num() == 1;
}

bool FLevelCollectionModel::AreAnyLevelsSelected() const
{
	return SelectedLevelsList.Num() > 0;
}

bool FLevelCollectionModel::AreAllSelectedLevelsUserManaged() const
{
	for (int32 LevelIdx = 0; LevelIdx < SelectedLevelsList.Num(); LevelIdx++)
	{
		if (!SelectedLevelsList[LevelIdx]->IsUserManaged())
		{
			return false;
		}
	}

	return AreAnyLevelsSelected();
}

bool FLevelCollectionModel::AreAllSelectedLevelsLoaded() const
{
	for (int32 LevelIdx = 0; LevelIdx < SelectedLevelsList.Num(); LevelIdx++)
	{
		if (SelectedLevelsList[LevelIdx]->IsLoaded() == false)
		{
			return false;
		}
	}

	return AreAnyLevelsSelected();
}

bool FLevelCollectionModel::AreAnySelectedLevelsLoaded() const
{
	return !AreAllSelectedLevelsUnloaded();
}

bool FLevelCollectionModel::AreAllSelectedLevelsUnloaded() const
{
	for (int32 LevelIdx = 0; LevelIdx < SelectedLevelsList.Num(); LevelIdx++)
	{
		if (SelectedLevelsList[LevelIdx]->IsLoaded() == true)
		{
			return false;
		}
	}

	return true;
}

bool FLevelCollectionModel::AreAnySelectedLevelsUnloaded() const
{
	return !AreAllSelectedLevelsLoaded();
}

bool FLevelCollectionModel::AreAllSelectedLevelsEditable() const
{
	for (auto It = SelectedLevelsList.CreateConstIterator(); It; ++It)
	{
		if ((*It)->IsEditable() == false)
		{
			return false;
		}
	}
	
	return AreAnyLevelsSelected();
}

bool FLevelCollectionModel::AreAllSelectedLevelsEditableAndNotPersistent() const
{
	for (auto It = SelectedLevelsList.CreateConstIterator(); It; ++It)
	{
		if ((*It)->IsEditable() == false || (*It)->IsPersistent() || !(*It)->IsUserManaged())
		{
			return false;
		}
	}
	
	return AreAnyLevelsSelected();
}

bool FLevelCollectionModel::AreAllSelectedLevelsEditableAndVisible() const
{
	for (auto It = SelectedLevelsList.CreateConstIterator(); It; ++It)
	{
		if ((*It)->IsEditable() == false ||
			(*It)->IsVisibleInEditor() == false)
		{
			return false;
		}
	}
	
	return AreAnyLevelsSelected();
}

bool FLevelCollectionModel::AreAnySelectedLevelsEditable() const
{
	for (auto It = SelectedLevelsList.CreateConstIterator(); It; ++It)
	{
		if ((*It)->IsEditable() == true)
		{
			return true;
		}
	}
	
	return false;
}

bool FLevelCollectionModel::AreAnySelectedLevelsEditableAndVisible() const
{
	for (auto It = SelectedLevelsList.CreateConstIterator(); It; ++It)
	{
		if ((*It)->IsEditable() == true && 
			(*It)->IsVisibleInEditor() == true)
		{
			return true;
		}
	}
	
	return false;
}

bool FLevelCollectionModel::CanExecuteGameVisibilityCommandsForSelectedLevels() const
{
	return AreAnySelectedLevelsLoaded() && CanExecuteGameVisibilityCommands();
}

bool FLevelCollectionModel::CanExecuteGameVisibilityCommands() const
{
	return !WorldHierarchy::IsInPie();
}

bool FLevelCollectionModel::IsSelectedLevelEditable() const
{
	if (SelectedLevelsList.Num() == 1)
	{
		return SelectedLevelsList[0]->IsEditable();
	}
	
	return false;
}

bool FLevelCollectionModel::IsNewLightingScenarioState(bool bExistingState) const
{
	if (SelectedLevelsList.Num() == 1)
	{
		return SelectedLevelsList[0]->IsLightingScenario() != bExistingState;
	}
	
	return false;
}

void FLevelCollectionModel::SetIsLightingScenario(bool bNewLightingScenario)
{
	if (SelectedLevelsList.Num() == 1)
	{
		SelectedLevelsList[0]->SetIsLightingScenario(bNewLightingScenario);
	}
}

bool FLevelCollectionModel::AreAnySelectedLevelsDirty() const
{
	for (auto It = SelectedLevelsList.CreateConstIterator(); It; ++It)
	{
		if ((*It)->IsLoaded() == true && (*It)->IsDirty() == true)
		{
			return true;
		}
	}
	
	return false;
}

bool FLevelCollectionModel::AreActorsSelected() const
{
	return GEditor->GetSelectedActorCount() > 0;
}

bool FLevelCollectionModel::CanConvertAnyLevelToExternalActors(bool bExternal) const
{
	if (SelectedLevelsList.Num())
	{
		for (const TSharedPtr<FLevelModel>& LevelModel : SelectedLevelsList)
		{
			if (!LevelModel->CanConvertLevelToExternalActors(bExternal))
			{
				return false;
			}
		}
		return true;
	}
	return false;
}

bool FLevelCollectionModel::GetDisplayPathsState() const
{
	return bDisplayPaths;
}

void FLevelCollectionModel::SetDisplayPathsState(bool InDisplayPaths)
{
	bDisplayPaths = InDisplayPaths;

	for (auto It = AllLevelsList.CreateIterator(); It; ++It)
	{
		(*It)->UpdateDisplayName();
	}
}

bool FLevelCollectionModel::GetDisplayActorsCountState() const
{
	return bDisplayActorsCount;
}

void FLevelCollectionModel::SetDisplayActorsCountState(bool InDisplayActorsCount)
{
	bDisplayActorsCount = InDisplayActorsCount;

	for (auto It = AllLevelsList.CreateIterator(); It; ++It)
	{
		(*It)->UpdateDisplayName();
	}
}

void FLevelCollectionModel::BroadcastSelectionChanged()
{
	SelectionChanged.Broadcast();
}

void FLevelCollectionModel::BroadcastCollectionChanged()
{
	CollectionChanged.Broadcast();
}
		
void FLevelCollectionModel::BroadcastHierarchyChanged()
{
	HierarchyChanged.Broadcast();
}

void FLevelCollectionModel::BroadcastPreLevelsUnloaded()
{
	PreLevelsUnloaded.Broadcast();
}

void FLevelCollectionModel::BroadcastPostLevelsUnloaded()
{
	PostLevelsUnloaded.Broadcast();
}

double FLevelCollectionModel::EditableAxisLength()
{ 
	return HALF_WORLD_MAX; 
};

FBox FLevelCollectionModel::EditableWorldArea()
{
	FVector::FReal AxisLength = EditableAxisLength();
	
	return FBox(	
		FVector(-AxisLength, -AxisLength, -AxisLength), 
		FVector(+AxisLength, +AxisLength, +AxisLength)
		);
}

void FLevelCollectionModel::SCCCheckOut(const FLevelModelList& InList)
{
	TArray<FString> FilenamesToCheckOut = GetFilenamesList(InList);
	
	// Update the source control status of all potentially relevant packages
	ISourceControlModule::Get().GetProvider().Execute(
		ISourceControlOperation::Create<FUpdateStatus>(), FilenamesToCheckOut
		);

	// Now check them out
	FEditorFileUtils::CheckoutPackages(FilenamesToCheckOut);
}

void FLevelCollectionModel::SCCCheckIn(const FLevelModelList& InList)
{
	TArray<UPackage*> PackagesToCheckIn = GetPackagesList(InList);
	TArray<FString> FilenamesToCheckIn = GetFilenamesList(InList);
	
	// Prompt the user to ask if they would like to first save any dirty packages they are trying to check-in
	const auto UserResponse = FEditorFileUtils::PromptForCheckoutAndSave(PackagesToCheckIn, true, true);

	// If the user elected to save dirty packages, but one or more of the packages failed to save properly OR if the user
	// canceled out of the prompt, don't follow through on the check-in process
	const bool bShouldProceed = UserResponse == FEditorFileUtils::EPromptReturnCode::PR_Success || 
								UserResponse == FEditorFileUtils::EPromptReturnCode::PR_Declined;
	if (bShouldProceed)
	{
		const bool bUseSourceControlStateCache = false;
		FSourceControlWindows::PromptForCheckin(bUseSourceControlStateCache, FilenamesToCheckIn);
	}
	else
	{
		// If a failure occurred, alert the user that the check-in was aborted. This warning shouldn't be necessary if the user cancelled
		// from the dialog, because they obviously intended to cancel the whole operation.
		if (UserResponse == FEditorFileUtils::EPromptReturnCode::PR_Failure)
		{
			FMessageDialog::Open(EAppMsgType::Ok, 
				NSLOCTEXT("UnrealEd", "SCC_Checkin_Aborted", "Check-in aborted as a result of save failure.")
				);
		}
	}
}

void FLevelCollectionModel::SCCOpenForAdd(const FLevelModelList& InList)
{
	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	TArray<FString>		FilenamesList = GetFilenamesList(InList);
	TArray<FString>		FilenamesToAdd;
	TArray<UPackage*>	PackagesToSave;

	for (auto It = FilenamesList.CreateConstIterator(); It; ++It)
	{
		const FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(*It, EStateCacheUsage::Use);
		if (SourceControlState.IsValid() && !SourceControlState->IsSourceControlled())
		{
			FilenamesToAdd.Add(*It);

			// Make sure the file actually exists on disk before adding it
			FString LongPackageName = FPackageName::FilenameToLongPackageName(*It);
			if (!FPackageName::DoesPackageExist(LongPackageName))
			{
				UPackage* Package = FindPackage(NULL, *LongPackageName);
				if (Package)
				{
					PackagesToSave.Add(Package);
				}
			}
		}
	}

	if (FilenamesToAdd.Num() > 0)
	{
		// If any of the packages are new, save them now
		if (PackagesToSave.Num() > 0)
		{
			const bool bCheckDirty = false;
			const bool bPromptToSave = false;
			const auto Return = FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, bCheckDirty, bPromptToSave);
		}

		SourceControlProvider.Execute(ISourceControlOperation::Create<FMarkForAdd>(), FilenamesToAdd);
	}
}

void FLevelCollectionModel::SCCHistory(const FLevelModelList& InList)
{
	// This is odd, why SCC needs package names, instead of filenames? 
	TArray<FString> PackageNames;
	for (auto It = InList.CreateConstIterator(); It; ++It)
	{
		if ((*It)->HasValidPackage())
		{
			PackageNames.Add((*It)->GetLongPackageName().ToString());
		}
	}

	FSourceControlWindows::DisplayRevisionHistory(PackageNames);
}

void FLevelCollectionModel::SCCRefresh(const FLevelModelList& InList)
{
	if(ISourceControlModule::Get().IsEnabled())
	{
		ISourceControlModule::Get().QueueStatusUpdate(GetFilenamesList(InList));
	}
}

void FLevelCollectionModel::SCCDiffAgainstDepot(const FLevelModelList& InList, UEditorEngine* InEditor)
{
	// Load the asset registry module
	FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");

	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

	// Iterate over each selected asset
	for (auto It = InList.CreateConstIterator(); It; ++It)
	{
		ULevel* Level = (*It)->GetLevelObject();
		if (Level == NULL)
		{
			return;
		}
		
		UPackage* OriginalPackage = Level->GetOutermost();
		FString PackageName = OriginalPackage->GetName();

		// Make sure our history is up to date
		auto UpdateStatusOperation = ISourceControlOperation::Create<FUpdateStatus>();
		UpdateStatusOperation->SetUpdateHistory(true);
		SourceControlProvider.Execute(UpdateStatusOperation, OriginalPackage);

		// Get the SCC state
		FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(
			OriginalPackage, EStateCacheUsage::Use
			);

		// If the level is in SCC.
		if (SourceControlState.IsValid() && SourceControlState->IsSourceControlled())
		{
			// Get the file name of package
			FString RelativeFileName;
			if(FPackageName::DoesPackageExist(PackageName, &RelativeFileName))
			{
				if (SourceControlState->GetHistorySize() > 0)
				{
					auto Revision = SourceControlState->GetHistoryItem(0);
					check(Revision.IsValid());

					// Get the head revision of this package from source control
					FString AbsoluteFileName = FPaths::ConvertRelativePathToFull(RelativeFileName);
					FString TempFileName;
					if (UPackage* OldPackage = DiffUtils::LoadPackageForDiff(Revision))
					{
						// Try and load that package
						FText NotMapReason;
						if(InEditor->PackageIsAMapFile(*TempFileName, NotMapReason))
						{
							/* Set the revision information*/
							UPackage* Package = OriginalPackage;

							FRevisionInfo OldRevision;
							OldRevision.Changelist = Revision->GetCheckInIdentifier();
							OldRevision.Date = Revision->GetDate();
							OldRevision.Revision = Revision->GetRevision();

							FRevisionInfo NewRevision; 
							NewRevision.Revision = TEXT("");

							// Dump assets to temp text files
							FString OldTextFilename = AssetToolsModule.Get().DumpAssetToTempFile(OldPackage);
							FString NewTextFilename = AssetToolsModule.Get().DumpAssetToTempFile(OriginalPackage);
							FString DiffCommand = GetDefault<UEditorLoadingSavingSettings>()->TextDiffToolPath.FilePath;

							AssetToolsModule.Get().CreateDiffProcess(DiffCommand, OldTextFilename, NewTextFilename);
							AssetToolsModule.Get().DiffAssets(OldPackage, OriginalPackage, OldRevision, NewRevision);
						}
					}
				}
			} 
		}
	}
}

TArray<FName> FLevelCollectionModel::GetPackageNamesList(const FLevelModelList& InList)
{
	TArray<FName> ResultList;
	
	for (auto It = InList.CreateConstIterator(); It; ++It)
	{
		if ((*It)->HasValidPackage())
		{
			ResultList.Add((*It)->GetLongPackageName());
		}
	}

	return ResultList;
}

TArray<FString> FLevelCollectionModel::GetFilenamesList(const FLevelModelList& InList)
{
	TArray<FString> ResultList;
	
	for (auto It = InList.CreateConstIterator(); It; ++It)
	{
		if ((*It)->HasValidPackage())
		{
			ResultList.Add((*It)->GetPackageFileName());
		}
	}

	return ResultList;
}

TArray<UPackage*> FLevelCollectionModel::GetPackagesList(const FLevelModelList& InList)
{
	TArray<UPackage*> ResultList;
	
	for (auto It = InList.CreateConstIterator(); It; ++It)
	{
		ULevel* Level = (*It)->GetLevelObject();
		if (Level)
		{
			ResultList.Add(Level->GetOutermost());
		}
	}

	return ResultList;
}
	
TArray<ULevel*> FLevelCollectionModel::GetLevelObjectList(const FLevelModelList& InList)
{
	TArray<ULevel*> ResultList;
	
	for (auto It = InList.CreateConstIterator(); It; ++It)
	{
		ULevel* Level = (*It)->GetLevelObject();
		if (Level)
		{
			ResultList.Add(Level);
		}
	}

	return ResultList;
}

FLevelModelList FLevelCollectionModel::GetLoadedLevels(const FLevelModelList& InList)
{
	FLevelModelList ResultList;
	for (auto It = InList.CreateConstIterator(); It; ++It)
	{
		if ((*It)->IsLoaded())
		{
			ResultList.Add(*It);
		}
	}
	
	return ResultList;
}

FLevelModelList FLevelCollectionModel::GetLevelsHierarchy(const FLevelModelList& InList)
{
	struct FHierarchyCollector : public FLevelModelVisitor
	{
		virtual void Visit(FLevelModel& Item) override
		{
			ResultList.AddUnique(Item.AsShared());
		}

		FLevelModelList ResultList;
	};

	FHierarchyCollector HierarchyCollector;
	for (auto It = InList.CreateConstIterator(); It; ++It)
	{
		(*It)->Accept(HierarchyCollector);
	}
	
	return HierarchyCollector.ResultList;
}

FBox FLevelCollectionModel::GetLevelsBoundingBox(const FLevelModelList& InList, bool bIncludeChildren)
{
	FBox TotalBounds(ForceInit);
	for (auto It = InList.CreateConstIterator(); It; ++It)
	{
		if (bIncludeChildren)
		{
			TotalBounds+= GetVisibleLevelsBoundingBox((*It)->GetChildren(), bIncludeChildren);
		}
		
		TotalBounds+= (*It)->GetLevelBounds();
	}

	return TotalBounds;
}

FBox FLevelCollectionModel::GetVisibleLevelsBoundingBox(const FLevelModelList& InList, bool bIncludeChildren)
{
	FBox TotalBounds(ForceInit);
	for (auto It = InList.CreateConstIterator(); It; ++It)
	{
		if (bIncludeChildren)
		{
			TotalBounds+= GetVisibleLevelsBoundingBox((*It)->GetChildren(), bIncludeChildren);
		}
		
		if ((*It)->IsVisibleInEditor())
		{
			TotalBounds+= (*It)->GetLevelBounds();
		}
	}

	return TotalBounds;
}

const TSharedRef<FUICommandList> FLevelCollectionModel::GetCommandList() const
{
	return CommandList;
}

const FString ConfigIniSection = TEXT("WorldBrowser");

void FLevelCollectionModel::LoadSettings()
{
	// Display paths
	bool bDisplayPathsSetting = false;
	GConfig->GetBool(*ConfigIniSection, TEXT("DisplayPaths"), bDisplayPathsSetting, GEditorPerProjectIni);
	SetDisplayPathsState(bDisplayPathsSetting);

	// Display actors count
	bool bDisplayActorsCountSetting = false;
	GConfig->GetBool(*ConfigIniSection, TEXT("DisplayActorsCount"), bDisplayActorsCountSetting, GEditorPerProjectIni);
	SetDisplayActorsCountState(bDisplayActorsCountSetting);
}
	
void FLevelCollectionModel::SaveSettings()
{
	// Display paths
	GConfig->SetBool(*ConfigIniSection, TEXT("DisplayPaths"), GetDisplayPathsState(), GEditorPerProjectIni);

	// Display actors count
	GConfig->SetBool(*ConfigIniSection, TEXT("DisplayActorsCount"), GetDisplayActorsCountState(), GEditorPerProjectIni);
}

void FLevelCollectionModel::RefreshBrowser_Executed()
{
	PopulateLevelsList();
}

void FLevelCollectionModel::LoadSelectedLevels_Executed()
{
	LoadLevels(GetSelectedLevels());
}

void FLevelCollectionModel::UnloadSelectedLevels_Executed()
{
	UnloadLevels(GetSelectedLevels());
}

void FLevelCollectionModel::OnSCCCheckOut()
{
	SCCCheckOut(GetSelectedLevels());
}

void FLevelCollectionModel::OnSCCCheckIn()
{
	SCCCheckIn(GetSelectedLevels());
}

void FLevelCollectionModel::OnSCCOpenForAdd()
{
	SCCOpenForAdd(GetSelectedLevels());
}

void FLevelCollectionModel::OnSCCHistory()
{
	SCCHistory(GetSelectedLevels());
}

void FLevelCollectionModel::OnSCCRefresh()
{
	SCCRefresh(GetSelectedLevels());
}

void FLevelCollectionModel::OnSCCDiffAgainstDepot()
{
	SCCDiffAgainstDepot(GetSelectedLevels(), GEditor);
}

void FLevelCollectionModel::OnSCCConnect() const
{
	ISourceControlModule::Get().ShowLoginDialog(FSourceControlLoginClosed(), ELoginWindowMode::Modeless);
}

void FLevelCollectionModel::SaveSelectedLevels_Executed()
{
	SaveLevels(GetSelectedLevels());
}

void FLevelCollectionModel::SaveSelectedLevelAs_Executed()
{
	if (SelectedLevelsList.Num() > 0)
	{
		ULevel* Level = SelectedLevelsList[0]->GetLevelObject();
		if (Level)
		{
			FEditorFileUtils::SaveLevelAs(Level);
		}
	}
}

void FLevelCollectionModel::MigrateSelectedLevels_Executed()
{
	// Gather the package names for the levels
	TArray<FName> PackageNames = GetPackageNamesList(GetSelectedLevels());
	FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");
	AssetToolsModule.Get().MigratePackages(PackageNames);
}

void FLevelCollectionModel::SelectAllLevels_Executed()
{
	SetSelectedLevels(FilteredLevelsList);
}

void FLevelCollectionModel::DeselectAllLevels_Executed()
{
	FLevelModelList NoLevels;
	SetSelectedLevels(NoLevels);
}

void FLevelCollectionModel::InvertSelection_Executed()
{
	FLevelModelList InvertedLevels;

	for (auto It = FilteredLevelsList.CreateIterator(); It; ++It)
	{
		if (!SelectedLevelsList.Contains(*It))
		{
			InvertedLevels.Add(*It);
		}
	}

	SetSelectedLevels(InvertedLevels);
}

void FLevelCollectionModel::ShowInEditorSelectedLevels_Executed()
{
	ShowLevelsInEditor(GetSelectedLevels());
}

void FLevelCollectionModel::HideInEditorSelectedLevels_Executed()
{
	HideLevelsInEditor(GetSelectedLevels());
}

void FLevelCollectionModel::ShowInEditorOnlySelectedLevels_Executed()
{
	//stash off a copy of the original array, as setting visibility can destroy the selection
	FLevelModelList SelectedLevelsCopy = GetSelectedLevels();
	
	InvertSelection_Executed();
	HideInEditorSelectedLevels_Executed();
	SetSelectedLevels(SelectedLevelsCopy);
	ShowInEditorSelectedLevels_Executed();
}

void FLevelCollectionModel::ShowInEditorAllButSelectedLevels_Executed()
{
	//stash off a copy of the original array, as setting visibility can destroy the selection
	FLevelModelList SelectedLevelsCopy = GetSelectedLevels();

	InvertSelection_Executed();
	ShowInEditorSelectedLevels_Executed();
	SetSelectedLevels(SelectedLevelsCopy);
	HideInEditorSelectedLevels_Executed();
}


void FLevelCollectionModel::ShowInEditorAllLevels_Executed()
{
	ShowLevelsInEditor(GetFilteredLevels());
}

void FLevelCollectionModel::HideInEditorAllLevels_Executed()
{
	HideLevelsInEditor(GetFilteredLevels());
}

void FLevelCollectionModel::ShowInGameSelectedLevels_Executed()
{
	ShowLevelsInGame(GetSelectedLevels());
}

void FLevelCollectionModel::HideInGameSelectedLevels_Executed()
{
	HideLevelsInGame(GetSelectedLevels());
}

void FLevelCollectionModel::ShowInGameOnlySelectedLevels_Executed()
{
	//stash off a copy of the original array, as setting visibility can destroy the selection
	FLevelModelList SelectedLevelsCopy = GetSelectedLevels();
	
	InvertSelection_Executed();
	HideInGameSelectedLevels_Executed();
	SetSelectedLevels(SelectedLevelsCopy);
	ShowInGameSelectedLevels_Executed();
}

void FLevelCollectionModel::ShowInGameAllButSelectedLevels_Executed()
{
	//stash off a copy of the original array, as setting visibility can destroy the selection
	FLevelModelList SelectedLevelsCopy = GetSelectedLevels();

	InvertSelection_Executed();
	ShowInGameSelectedLevels_Executed();
	SetSelectedLevels(SelectedLevelsCopy);
	HideInGameSelectedLevels_Executed();
}

void FLevelCollectionModel::ShowInGameAllLevels_Executed()
{
	ShowLevelsInGame(GetFilteredLevels());
}

void FLevelCollectionModel::HideInGameAllLevels_Executed()
{
	HideLevelsInGame(GetFilteredLevels());
}

void FLevelCollectionModel::LockSelectedLevels_Executed()
{
	LockLevels(GetSelectedLevels());
}

void FLevelCollectionModel::UnlockSelectedLevels_Executed()
{
	UnlockLevels(GetSelectedLevels());
}

void FLevelCollectionModel::LockOnlySelectedLevels_Executed()
{
	//stash off a copy of the original array, as setting visibility can destroy the selection
	FLevelModelList SelectedLevelsCopy = GetSelectedLevels();

	InvertSelection_Executed();
	UnlockSelectedLevels_Executed();
	SetSelectedLevels(SelectedLevelsCopy);
	LockSelectedLevels_Executed();
}

void FLevelCollectionModel::LockAllButSelectedLevels_Executed()
{
	//stash off a copy of the original array, as setting visibility can destroy the selection
	FLevelModelList SelectedLevelsCopy = GetSelectedLevels();

	InvertSelection_Executed();
	LockSelectedLevels_Executed();
	SetSelectedLevels(SelectedLevelsCopy);
	UnlockSelectedLevels_Executed();
}

void FLevelCollectionModel::LockAllLevels_Executed()
{
	if (!IsReadOnly())
	{
		const FScopedTransaction Transaction(LOCTEXT("LockAllLevels", "Lock All Levels"));
		LockLevels(GetFilteredLevels());
	}
}

void FLevelCollectionModel::UnlockAllLevels_Executed()
{
	if (!IsReadOnly())
	{
		const FScopedTransaction Transaction(LOCTEXT("UnlockAllLevels", "Unlock All Levels"));
		UnlockLevels(GetFilteredLevels());
	}
}

void FLevelCollectionModel::ToggleReadOnlyLevels_Executed()
{
	//We are about to lock some Levels, deselect all actor and surfaces from the read only levels
	if (!GEditor->bLockReadOnlyLevels)
	{
		DeselectActorsInAllReadOnlyLevel(GetAllLevels());
		DeselectSurfaceInAllReadOnlyLevel(GetAllLevels());
		// Tell the editor selection status was changed.
		GEditor->NoteSelectionChange();
	}

	GEditor->bLockReadOnlyLevels = !GEditor->bLockReadOnlyLevels;
}

void FLevelCollectionModel::MakeLevelCurrent_Executed()
{
	check( SelectedLevelsList.Num() == 1 );
	SelectedLevelsList[0]->MakeLevelCurrent();
}

void FLevelCollectionModel::FindInContentBrowser_Executed()
{
	TArray<UObject*> Objects;
	for (auto It = SelectedLevelsList.CreateConstIterator(); It; ++It)
	{
		ULevel* Level = (*It)->GetLevelObject();
		if (Level)
		{
			UObject* LevelOuter = Level->GetOuter();
			if (LevelOuter)
			{
				// Search for the level's outer (the UWorld) as this is the actual asset shown by the content browser
				Objects.AddUnique(LevelOuter);
			}
		}
	}

	GEditor->SyncBrowserToObjects(Objects);
}

bool FLevelCollectionModel::IsValidFindInContentBrowser()
{
	return true;
}

void FLevelCollectionModel::MoveActorsToSelected_Executed()
{
	MakeLevelCurrent_Executed();

	const FScopedTransaction Transaction(LOCTEXT("MoveSelectedActorsToSelectedLevel", "Move Selected Actors to Level"));

	// Redirect selected foliage actor to use the MoveActorFoliageInstancesToLevel functionality as we can't move the foliage actor only instances
	USelection* SelectedActors = GEditor->GetSelectedActors();
	for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
	{
		AInstancedFoliageActor* Actor = Cast<AInstancedFoliageActor>(*Iter);

		if (Actor != nullptr)
		{
			FFoliageEditUtility::MoveActorFoliageInstancesToLevel(GetWorld()->GetCurrentLevel(), Actor);
		}
	}

	UEditorLevelUtils::MoveSelectedActorsToLevel(GetWorld()->GetCurrentLevel());

	RequestUpdateAllLevels();
}

void FLevelCollectionModel::MoveFoliageToSelected_Executed()
{
	if (GetSelectedLevels().Num() == 1)
	{
		ULevel* TargetLevel = GetSelectedLevels()[0]->GetLevelObject();

		// Need to only permit this action when the foliage mode is open as the selection is being done there
		if (GLevelEditorModeTools().IsModeActive(FBuiltinEditorModes::EM_Foliage))
		{
			IFoliageEditModule& FoliageModule = FModuleManager::GetModuleChecked<IFoliageEditModule>("FoliageEdit");
			FoliageModule.MoveSelectedFoliageToLevel(TargetLevel);
		}
	}
}

void FLevelCollectionModel::SelectActors_Executed()
{
	//first clear any existing actor selection
	const FScopedTransaction Transaction( LOCTEXT("SelectActors", "Select Actors in Level") );
	GEditor->GetSelectedActors()->Modify();
	GEditor->SelectNone( false, true );

	for(auto It = SelectedLevelsList.CreateConstIterator(); It; ++It)
	{
		(*It)->SelectActors(/*bSelect*/ true, /*bNotify*/ true, /*bSelectEvenIfHidden*/ true);
	}
}

void FLevelCollectionModel::DeselectActors_Executed()
{
	const FScopedTransaction Transaction( LOCTEXT("DeselectActors", "Deselect Actors in Level") );
	
	for(auto It = SelectedLevelsList.CreateConstIterator(); It; ++It)
	{
		(*It)->SelectActors(/*bSelect*/ false, /*bNotify*/ true, /*bSelectEvenIfHidden*/ true);
	}
}

void FLevelCollectionModel::ConvertLevelToExternalActors_Executed(bool bExternal)
{
	FScopedTransaction Transaction(LOCTEXT("WorldUseExternalActors", "Change World Use External Actors"));
	for (const TSharedPtr<FLevelModel>& LevelModel : SelectedLevelsList)
	{
		LevelModel->ConvertLevelToExternalActors(bExternal);
	}
}

void FLevelCollectionModel::ExpandSelectedItems_Executed()
{
	struct FExpandLevelVisitor : public FLevelModelVisitor
	{
		virtual void Visit(FLevelModel& Item) override { Item.SetLevelExpansionFlag(true); }
	} Expander;
	
	for (TSharedPtr<FLevelModel> LevelModel: SelectedLevelsList)
	{
		LevelModel->Accept(Expander);
	}

	BroadcastHierarchyChanged();
}


void FLevelCollectionModel::FillLockSubMenu(FMenuBuilder& InMenuBuilder)
{
	const FLevelCollectionCommands& Commands = FLevelCollectionCommands::Get();

	InMenuBuilder.AddMenuEntry( Commands.World_LockSelectedLevels );
	InMenuBuilder.AddMenuEntry( Commands.World_UnlockSelectedLevels );
	InMenuBuilder.AddMenuEntry( Commands.World_LockOnlySelectedLevels );
	InMenuBuilder.AddMenuEntry( Commands.World_LockAllButSelectedLevels );
	InMenuBuilder.AddMenuEntry( Commands.World_LockAllLevels );
	InMenuBuilder.AddMenuEntry( Commands.World_UnlockAllLevels );

	if (GEditor->bLockReadOnlyLevels)
	{
		InMenuBuilder.AddMenuEntry( Commands.World_UnlockReadOnlyLevels );
	}
	else
	{
		InMenuBuilder.AddMenuEntry( Commands.World_LockReadOnlyLevels );
	}
}

void FLevelCollectionModel::FillEditorVisibilitySubMenu(FMenuBuilder& InMenuBuilder)
{
	const FLevelCollectionCommands& Commands = FLevelCollectionCommands::Get();

	InMenuBuilder.AddMenuEntry( Commands.World_ShowInEditorSelectedLevels );
	InMenuBuilder.AddMenuEntry( Commands.World_HideInEditorSelectedLevels );
	InMenuBuilder.AddMenuEntry( Commands.World_ShowInEditorOnlySelectedLevels );
	InMenuBuilder.AddMenuEntry( Commands.World_ShowInEditorAllButSelectedLevels );
	InMenuBuilder.AddMenuEntry( Commands.World_ShowInEditorAllLevels );
	InMenuBuilder.AddMenuEntry( Commands.World_HideInEditorAllLevels );
}

void FLevelCollectionModel::FillGameVisibilitySubMenu(FMenuBuilder& InMenuBuilder)
{
	const FLevelCollectionCommands& Commands = FLevelCollectionCommands::Get();

	InMenuBuilder.AddMenuEntry( Commands.World_ShowInGameSelectedLevels );
	InMenuBuilder.AddMenuEntry( Commands.World_HideInGameSelectedLevels );
	InMenuBuilder.AddMenuEntry( Commands.World_ShowInGameOnlySelectedLevels );
	InMenuBuilder.AddMenuEntry( Commands.World_ShowInGameAllButSelectedLevels );
	InMenuBuilder.AddMenuEntry( Commands.World_ShowInGameAllLevels );
	InMenuBuilder.AddMenuEntry( Commands.World_HideInGameAllLevels );
}

void FLevelCollectionModel::FillSourceControlSubMenu(FMenuBuilder& InMenuBuilder)
{
	const FLevelCollectionCommands& Commands = FLevelCollectionCommands::Get();
	
	if (CanExecuteSCC())
	{
		if (CanExecuteSCCCheckOut())
		{
			InMenuBuilder.AddMenuEntry( Commands.SCCCheckOut );
		}

		if (CanExecuteSCCOpenForAdd())
		{
			InMenuBuilder.AddMenuEntry( Commands.SCCOpenForAdd );
		}

		if (CanExecuteSCCCheckIn())
		{
			InMenuBuilder.AddMenuEntry( Commands.SCCCheckIn );
		}

		InMenuBuilder.AddMenuEntry( Commands.SCCRefresh );
		InMenuBuilder.AddMenuEntry( Commands.SCCHistory );
		InMenuBuilder.AddMenuEntry( Commands.SCCDiffAgainstDepot );
	}
	else
	{
		InMenuBuilder.AddMenuEntry( Commands.SCCConnect );
	}
}

void FLevelCollectionModel::DeselectActorsInAllReadOnlyLevel(const FLevelModelList& InLevelList)
{
	const FScopedTransaction Transaction(LOCTEXT("DeselectActorsInReadOnlyLevel", "Deselect Actors in all read only Level"));

	for (auto It = InLevelList.CreateConstIterator(); It; ++It)
	{
		if ((*It)->IsFileReadOnly())
		{
			(*It)->SelectActors(/*bSelect*/ false, /*bNotify*/ true, /*bSelectEvenIfHidden*/ true);
		}
	}
}

void FLevelCollectionModel::DeselectSurfaceInAllReadOnlyLevel(const FLevelModelList& InLevelList)
{
	const FScopedTransaction Transaction(LOCTEXT("DeselectSurfacesInReadOnlyLevel", "Deselect Surfaces in all read only Level"));

	for (auto It = InLevelList.CreateConstIterator(); It; ++It)
	{
		if ((*It)->IsFileReadOnly())
		{
			(*It)->DeselectAllSurfaces();
		}
	}
}


void FLevelCollectionModel::OnLevelsCollectionChanged()
{
	UpdateAllLevels();
	PopulateFilteredLevelsList();

	BroadcastCollectionChanged();
}

void FLevelCollectionModel::OnLevelsSelectionChanged()
{
	if (bUpdatingLevelsSelection)
	{
		return;
	}
	
	TGuardValue<bool> UpdateGuard(bUpdatingLevelsSelection, true);
	
	// Pass the list we just created to the world to set the selection
	CurrentWorld->SetSelectedLevels(
		GetLevelObjectList(SelectedLevelsList)
		);

	// Request SC status update for selected levels
	ISourceControlModule::Get().QueueStatusUpdate(
		GetFilenamesList(SelectedLevelsList)
		);

	// Expand hierarchy to selected levels 
	for (auto It = SelectedLevelsList.CreateIterator(); It; ++It)
	{
		TSharedPtr<FLevelModel> ParentLevelModel = (*It)->GetParent();
		while (ParentLevelModel.IsValid())
		{
			ParentLevelModel->SetLevelExpansionFlag(true);
			ParentLevelModel = ParentLevelModel->GetParent();
		}
	}
		
	BroadcastSelectionChanged();
}

void FLevelCollectionModel::OnLevelsSelectionChangedOutside()
{
	if (!bUpdatingLevelsSelection)
	{
		SetSelectedLevelsFromWorld();
	}
}

void FLevelCollectionModel::OnLevelsHierarchyChanged()
{
	BroadcastHierarchyChanged();
}

void FLevelCollectionModel::OnLevelAddedToWorld(ULevel* InLevel, UWorld* InWorld)
{
	if (InWorld == GetWorld())
	{
		TSharedPtr<FLevelModel> LevelModel = FindLevelModel(InLevel->GetOutermost()->GetFName());
		if (LevelModel.IsValid())
		{
			LevelModel->OnLevelAddedToWorld(InLevel);
		}
	}
}

void FLevelCollectionModel::OnLevelRemovedFromWorld(ULevel* InLevel, UWorld* InWorld)
{
	if (InWorld == GetWorld())
	{
		TSharedPtr<FLevelModel> LevelModel = FindLevelModel(InLevel->GetOutermost()->GetFName());
		if (LevelModel.IsValid())
		{
			LevelModel->OnLevelRemovedFromWorld();
		}
	}
}

void FLevelCollectionModel::OnRedrawAllViewports()
{
	if (GShaderCompilingManager && GShaderCompilingManager->IsCompiling())
	{
		// Editor seems like still compiling shaders, do not request tiles redraw until all shaders complation is finished
		// Basically redraw only on last event
		return;
	}
	
	RequestRedrawAllLevels();
}

void FLevelCollectionModel::OnLevelActorAdded(AActor* InActor)
{
	if (InActor && 
		InActor->GetWorld() == CurrentWorld.Get()) // we care about our world only
	{
		bRequestedUpdateActorsCount = true;
	}
}

void FLevelCollectionModel::OnLevelActorDeleted(AActor* InActor)
{
	bRequestedUpdateActorsCount = true;
}

void FLevelCollectionModel::OnFilterChanged()
{
	PopulateFilteredLevelsList();
	BroadcastCollectionChanged();
}

void FLevelCollectionModel::CacheCanExecuteSourceControlVars() const
{
	bCanExecuteSCCCheckOut = false;
	bCanExecuteSCCOpenForAdd = false;
	bCanExecuteSCCCheckIn = false;
	bCanExecuteSCC = false;

	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	for (auto It = SelectedLevelsList.CreateConstIterator(); It; ++It)
	{
		if (ISourceControlModule::Get().IsEnabled() && SourceControlProvider.IsAvailable())
		{
			bCanExecuteSCC = true;
			
			ULevel* Level = (*It)->GetLevelObject();
			if (Level)
			{
				// Check the SCC state for each package in the selected paths
				FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(Level->GetOutermost(), EStateCacheUsage::Use);

				if (SourceControlState.IsValid())
				{
					if (SourceControlState->CanCheckout())
					{
						bCanExecuteSCCCheckOut = true;
					}
					else if (!SourceControlState->IsSourceControlled())
					{
						bCanExecuteSCCOpenForAdd = true;
					}
					else if (SourceControlState->IsCheckedOut() || SourceControlState->IsAdded())
					{
						bCanExecuteSCCCheckIn = true;
					}
				}
			}
		}

		if (bCanExecuteSCCCheckOut && 
			bCanExecuteSCCOpenForAdd && 
			bCanExecuteSCCCheckIn)
		{
			// All options are available, no need to keep iterating
			break;
		}
	}
}

bool FLevelCollectionModel::IsValidMoveActorsToLevel() const
{
	static bool bCachedIsValidActorMoveResult = false;
	if (bSelectionHasChanged)
	{
		bSelectionHasChanged = false;
		bCachedIsValidActorMoveResult = false;

		// We can only operate on a single selected level
		if ( SelectedLevelsList.Num() == 1 )
		{
			ULevel* Level = SelectedLevelsList[0]->GetLevelObject();
			if (Level)
			{
				// Allow the move if at least one actor is in another level
				USelection* SelectedActors = GEditor->GetSelectedActors();
				for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
				{
					AActor* Actor = CastChecked<AActor>(*Iter);
					if (Actor != nullptr)
					{
						if (Actor->GetLevel() != Level)
						{
							bCachedIsValidActorMoveResult = true;
							break;
						}
					}
				}
			}
		}
	}
			
	// if non of the selected actors are in the level, just check the level is unlocked
	return bCachedIsValidActorMoveResult && AreAllSelectedLevelsEditableAndVisible();
}

bool FLevelCollectionModel::IsValidMoveFoliageToLevel() const
{
	if (IsOneLevelSelected() && 
		AreAllSelectedLevelsEditableAndVisible() && 
		GLevelEditorModeTools().IsModeActive(FBuiltinEditorModes::EM_Foliage))
	{
		IFoliageEditModule& FoliageModule = FModuleManager::GetModuleChecked<IFoliageEditModule>("FoliageEdit");
		ULevel* TargetLevel = GetSelectedLevels()[0]->GetLevelObject();

		return FoliageModule.CanMoveSelectedFoliageToLevel(TargetLevel);
	}

	return false;
}

void FLevelCollectionModel::OnActorSelectionChanged(UObject* obj)
{
	OnActorOrLevelSelectionChanged();
}

void FLevelCollectionModel::OnActorOrLevelSelectionChanged()
{
	bSelectionHasChanged = true;
}


#undef LOCTEXT_NAMESPACE
