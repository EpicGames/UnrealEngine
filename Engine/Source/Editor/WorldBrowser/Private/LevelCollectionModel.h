// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "Framework/Commands/UICommandList.h"
#include "Engine/World.h"
#include "TickableEditorObject.h"
#include "WorldBrowserDragDrop.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/IFilter.h"
#include "LevelModel.h"

#include "Misc/FilterCollection.h"

#define UE_API WORLDBROWSER_API

class FMenuBuilder;
class IDetailsView;
class UEditorEngine;
class UMaterialInterface;

typedef IFilter< const FLevelModel* >				LevelFilter;
typedef TFilterCollection< const FLevelModel* >		LevelFilterCollection;

enum class EBuildHierarchyMenuFlags
{
	None,
	ShowGameVisibility = 1 << 0
};
ENUM_CLASS_FLAGS(EBuildHierarchyMenuFlags);

/** Interface for non-UI presentation logic for a world. */
class FLevelCollectionModel
	: public TSharedFromThis<FLevelCollectionModel>	
	, public FTickableEditorObject
{
public:
	
	DECLARE_EVENT_OneParam( FLevelCollectionModel, FOnNewItemAdded, TSharedPtr<FLevelModel>);
	DECLARE_EVENT( FLevelCollectionModel, FSimpleEvent );
	
	UE_API FLevelCollectionModel();
	UE_API virtual ~FLevelCollectionModel() override;

	/** FTickableEditorObject interface */
	UE_API virtual void Tick( float DeltaTime ) override;
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Always; }
	UE_API virtual TStatId GetStatId() const override;
	/** FTickableEditorObject interface */
	
	/**	@return	Whether level collection is read only now */
	UE_API bool IsReadOnly() const;
	
	/**	@return	Whether level collection is in PIE/SIE mode */
	UE_API bool IsSimulating() const;

	/**	@return	Current simulation world */
	UE_API UWorld* GetSimulationWorld() const;

	/**	@return	Current editor world */
	UWorld* GetWorld(bool bEvenIfPendingKill = false) const { return CurrentWorld.Get(bEvenIfPendingKill); }

	/** @return	Whether current world has world origin rebasing enabled */
	UE_API bool IsOriginRebasingEnabled() const;

	/** Current world size  */
	FIntPoint GetWorldSize() const { return WorldSize; }
	
	/**	@return	Root list of levels in hierarchy */
	UE_API FLevelModelList& GetRootLevelList();

	/**	@return	All level list managed by this level collection */
	UE_API const FLevelModelList& GetAllLevels() const;

	/**	@return	List of filtered levels */
	UE_API const FLevelModelList& GetFilteredLevels() const;

	/**	@return	Currently selected level list */
	UE_API const FLevelModelList& GetSelectedLevels() const;

	/** Adds a filter which restricts the Levels shown in UI */
	UE_API void AddFilter(const TSharedRef<LevelFilter>& InFilter);

	/** Removes a filter which restricted the Levels shown in UI */
	UE_API void RemoveFilter(const TSharedRef<LevelFilter>& InFilter);

	/**	@return	Whether level filtering is active now */
	UE_API bool IsFilterActive() const;
	
	/**	Iterates through level hierarchy with given Visitor */
	UE_API void IterateHierarchy(FLevelModelVisitor& Visitor);

	/**	Sets selected level list */
	UE_API void SetSelectedLevels(const FLevelModelList& InList);
	
	/**	Sets selection to a levels that is currently marked as selected in UWorld */
	UE_API void SetSelectedLevelsFromWorld();

	/**	@return	Found level model which represents specified level object */
	UE_API TSharedPtr<FLevelModel> FindLevelModel(ULevel* InLevel) const;

	/**	@return	Found level model with specified level package name */
	UE_API TSharedPtr<FLevelModel> FindLevelModel(const FName& PackageName) const;

	/**	Hides level in editor worlds */
	UE_API void HideLevelsInEditor(const FLevelModelList& InLevelList);

	/**	Shows level in editor worlds */
	UE_API void ShowLevelsInEditor(const FLevelModelList& InLevelList);

	/** Toggles the selected levels to a visible state; toggles all other levels to an invisible state */
	UE_API void ShowInEditorOnlySelectedLevels();

	/** Toggles the selected levels to an invisible state; toggles all other levels to a visible state */
	UE_API void ShowInEditorAllButSelectedLevels();
	
	/**	Hides level in game worlds */
	UE_API void HideLevelsInGame(const FLevelModelList& InLevelList);

	/**	Shows level in game worlds */
	UE_API void ShowLevelsInGame(const FLevelModelList& InLevelList);

	/** Toggles the selected levels to a visible state for game worlds; toggles all other levels to an invisible state */
	UE_API void ShowInGameOnlySelectedLevels();

	/** Toggles the selected levels to an invisible state for game worlds; toggles all other levels to a visible state */
	UE_API void ShowInGameAllButSelectedLevels();

	/**	Unlocks level in the world */
	UE_API void UnlockLevels(const FLevelModelList& InLevelList);
	/**	Locks level in the world */
	UE_API void LockLevels(const FLevelModelList& InLevelList);

	/** Toggles the selected levels to a locked state; toggles all other levels to an unlocked state */
	UE_API void LockOnlySelectedLevels();

	/** Toggles the selected levels to an unlocked state; toggles all other levels to a locked state */
	UE_API void LockAllButSelectedLevels();

	/**	Saves level to disk */
	UE_API void SaveLevels(const FLevelModelList& InLevelList);

	/**	Loads level from disk */
	UE_API void LoadLevels(const FLevelModelList& InLevelList);
	/**	Unloads levels from the editor */
	UE_API virtual void UnloadLevels(const FLevelModelList& InLevelList);

	/** Translate levels by specified delta */
	UE_API virtual void TranslateLevels(const FLevelModelList& InLevelList, FVector2D InAbsoluteDelta, bool bSnapDelta = true);
	
	/** Snaps translation delta */
	UE_API virtual FVector2D SnapTranslationDelta(const FLevelModelList& InLevelList, FVector2D InAbsoluteDelta, bool bBoundsSnapping, FVector2D::FReal SnappingValue);

	/**	Updates current translation delta, when user drags levels on minimap */
	UE_API virtual void UpdateTranslationDelta(const FLevelModelList& InLevelList, FVector2D InTranslationDelta, bool bBoundsSnapping, FVector2D::FReal SnappingValue);

	/** Attach levels as children to specified level */
	UE_API void AssignParent(const FLevelModelList& InLevels, TSharedPtr<FLevelModel> InParent);

	/** Adds all levels in worlds represented by the supplied world list as sublevels */
	UE_API virtual void AddExistingLevelsFromAssetData(const TArray<struct FAssetData>& WorldList);
			
	/**	Create drag drop operation for a selected level models */
	UE_API virtual TSharedPtr<WorldHierarchy::FWorldBrowserDragDropOp> CreateDragDropOp() const;

	/** Create a drag and drop operation for the specified level models */
	UE_API virtual TSharedPtr<WorldHierarchy::FWorldBrowserDragDropOp> CreateDragDropOp(const FLevelModelList& InLevels) const;
	
	/**	@return	Whether specified level passes all filters */
	UE_API virtual bool PassesAllFilters(const FLevelModel& InLevelModel) const;
	
	/**	Builds 'hierarchy' commands menu for a selected levels */
	UE_API virtual void BuildHierarchyMenu(FMenuBuilder& InMenuBuilder, EBuildHierarchyMenuFlags Flags) const;
	
	/**	Customize 'File' section in main menu  */
	UE_API virtual void CustomizeFileMainMenu(FMenuBuilder& InMenuBuilder) const;
		
	/**	@return	Player view in the PIE/Simulation world */
	UE_API virtual bool GetPlayerView(FVector& Location, FRotator& Rotation) const;

	/**	@return	Observer view in the Editor/Similuation world */
	UE_API virtual bool GetObserverView(FVector& Location, FRotator& Rotation) const;

	/**	Compares 2 levels by Z order */
	UE_API virtual bool CompareLevelsZOrder(TSharedPtr<FLevelModel> InA, TSharedPtr<FLevelModel> InB) const;

	/**	Registers level details customizations */
	UE_API virtual void RegisterDetailsCustomization(class FPropertyEditorModule& PropertyModule, TSharedPtr<class IDetailsView> InDetailsView);
	
	/**	Unregisters level details customizations */
	UE_API virtual void UnregisterDetailsCustomization(class FPropertyEditorModule& PropertyModule, TSharedPtr<class IDetailsView> InDetailsView);

	/** @return	Whether this level collection model is a tile world */
	virtual bool IsTileWorld() const { return false; };

	/** Returns true if this collection model will support folders */
	virtual bool HasFolderSupport() const { return false; }

	/** Rebuilds levels collection */
	UE_API void PopulateLevelsList();

	/** Rebuilds the list of filtered Levels */
	UE_API void PopulateFilteredLevelsList();

	/**	Request to update levels cached information */
	UE_API void RequestUpdateAllLevels();
	
	/**	Request to redraw all levels */
	UE_API void RequestRedrawAllLevels();

	/**	Updates all levels cached information */
	UE_API void UpdateAllLevels();

	/**	Redraws all levels */
	UE_API void RedrawAllLevels();

	/** Updates level actor count for all levels */
	UE_API void UpdateLevelActorsCount();

	/** @return	whether exactly one level is selected */
	UE_API bool IsOneLevelSelected() const;

	/** @return	whether at least one level is selected */
	UE_API bool AreAnyLevelsSelected() const;

	/** @return wether all selected levels are user managed */
	UE_API bool AreAllSelectedLevelsUserManaged() const;

	/** @return whether all the currently selected levels are loaded */
	UE_API bool AreAllSelectedLevelsLoaded() const;

	/** @return whether any of the currently selected levels is loaded */
	UE_API bool AreAnySelectedLevelsLoaded() const;
	
	/** @return whether all the currently selected levels are unloaded */
	UE_API bool AreAllSelectedLevelsUnloaded() const;
	
	/** @return whether any of the currently selected levels is unloaded */
	UE_API bool AreAnySelectedLevelsUnloaded() const;

	/** @return whether all the currently selected levels are editable */
	UE_API bool AreAllSelectedLevelsEditable() const;

	/** @return whether all the currently selected levels are editable and not persistent */
	UE_API bool AreAllSelectedLevelsEditableAndNotPersistent() const;

	/** @return whether all the currently selected levels are editable and visible*/
	UE_API bool AreAllSelectedLevelsEditableAndVisible() const;

	/** @return whether any of the currently selected levels is editable */
	UE_API bool AreAnySelectedLevelsEditable() const;

	/** @return whether any of the currently selected levels is editable and visible*/
	UE_API bool AreAnySelectedLevelsEditableAndVisible() const;

	/** @return Whether game visibility of the selected levels is allowed to be changed. */
	UE_API bool CanExecuteGameVisibilityCommandsForSelectedLevels() const;
	
	/** @return Whether game visibility of levels is allowed to be changed. */
	UE_API bool CanExecuteGameVisibilityCommands() const;
	
	/** @return whether currently only one level selected and it is editable */
	UE_API bool IsSelectedLevelEditable() const;

	/** @return whether currently only one level selected and a lighting scenario */
	UE_API bool IsNewLightingScenarioState(bool bExistingState) const;

	UE_API void SetIsLightingScenario(bool bNewLightingScenario);

	/** @return whether any of the currently selected levels is dirty */
	UE_API bool AreAnySelectedLevelsDirty() const;

	/** @return	whether at least one actor is selected */
	UE_API bool AreActorsSelected() const;

	/** @return whether any of the currently selected levels can be converted to the specified actor bExternal packaging */
	UE_API bool CanConvertAnyLevelToExternalActors(bool bExternal) const;

	/** @return whether moving the selected actors to the selected level is a valid action */
	UE_API bool IsValidMoveActorsToLevel() const;

	/** @return whether moving the selected foliage to the selected level is a valid action */
	UE_API bool IsValidMoveFoliageToLevel() const;

	/** delegate used to pickup when the selection has changed */
	UE_API void OnActorSelectionChanged(UObject* obj);

	/** Sets a flag to re-cache whether the selected actors move to the selected level is valid */
	UE_API void OnActorOrLevelSelectionChanged();

	/** @return	whether 'display paths' is enabled */
	UE_API bool GetDisplayPathsState() const;

	/** Sets 'display paths', whether to show long package name in level display name */
	UE_API void SetDisplayPathsState(bool bDisplayPaths);

	/** @return	whether 'display actors count' is enabled */
	UE_API bool GetDisplayActorsCountState() const;

	/** Sets 'display actors count', whether to show actors count next to level name */
	UE_API void SetDisplayActorsCountState(bool bDisplayActorsCount);

	/**	Broadcasts whenever items selection has changed */
	FSimpleEvent SelectionChanged;
	UE_API void BroadcastSelectionChanged();

	/**	Broadcasts whenever items collection has changed */
	FSimpleEvent CollectionChanged;
	UE_API void BroadcastCollectionChanged();
		
	/** Broadcasts whenever items hierarchy has changed */
	FSimpleEvent HierarchyChanged;
	UE_API void BroadcastHierarchyChanged();

	/** Broadcasts before levels are unloaded */
	FSimpleEvent PreLevelsUnloaded;
	UE_API void BroadcastPreLevelsUnloaded();

	/** Broadcasts after levels are unloaded */
	FSimpleEvent PostLevelsUnloaded;
	UE_API void BroadcastPostLevelsUnloaded();
	
	/** Editable world axis length  */
	static UE_API double EditableAxisLength();

	/** Editable world bounds */
	static UE_API FBox EditableWorldArea();

	/**  */
	static UE_API void SCCCheckOut(const FLevelModelList& InList);
	static UE_API void SCCCheckIn(const FLevelModelList& InList);
	static UE_API void SCCOpenForAdd(const FLevelModelList& InList);
	static UE_API void SCCHistory(const FLevelModelList& InList);
	static UE_API void SCCRefresh(const FLevelModelList& InList);
	static UE_API void SCCDiffAgainstDepot(const FLevelModelList& InList, UEditorEngine* InEditor);
	
	/** @return	List of valid level package names from a specified level model list*/
	static UE_API TArray<FName> GetPackageNamesList(const FLevelModelList& InList);
	
	/** @return	List of valid level package filenames from a specified level model list*/
	static UE_API TArray<FString> GetFilenamesList(const FLevelModelList& InList);
	
	/** @return	List of valid packages from a specified level model list*/
	static UE_API TArray<UPackage*> GetPackagesList(const FLevelModelList& InList);
	
	/** @return	List of valid level objects from a specified level model list*/
	static UE_API TArray<ULevel*> GetLevelObjectList(const FLevelModelList& InList);

	/** @return	List of loaded level models from a specified level model list*/
	static UE_API FLevelModelList GetLoadedLevels(const FLevelModelList& InList);

	/** @return	List of all level models found while traversing hierarchy of specified level models */
	static UE_API FLevelModelList GetLevelsHierarchy(const FLevelModelList& InList);

	/** @return	Total bounding box of specified level models */
	static UE_API FBox GetLevelsBoundingBox(const FLevelModelList& InList, bool bIncludeChildren);

	/** @return	Total bounding box of specified visible level models */
	static UE_API FBox GetVisibleLevelsBoundingBox(const FLevelModelList& InList, bool bIncludeChildren);

	/** @return	The UICommandList supported by this collection */
	UE_API const TSharedRef<FUICommandList> GetCommandList() const;

	/**  */
	UE_API void LoadSettings();
	
	/**  */
	UE_API void SaveSettings();

protected:
	/** Refreshes current cached data */
	UE_API void RefreshBrowser_Executed();
	
	/** Load selected levels to the world */
	UE_API void LoadSelectedLevels_Executed();

	/** Unload selected level from the world */
	UE_API void UnloadSelectedLevels_Executed();

	/** Make this Level the Current Level */
	UE_API void MakeLevelCurrent_Executed();

	/** Find selected levels in Content Browser */
	UE_API void FindInContentBrowser_Executed();

	/** Is FindInContentBrowser a valid action */
	UE_API bool IsValidFindInContentBrowser();

	/** Moves the selected actors to this level */
	UE_API void MoveActorsToSelected_Executed();

	/** Moves the selected foliage to this level */
	UE_API void MoveFoliageToSelected_Executed();

	/** Saves selected levels */
	UE_API void SaveSelectedLevels_Executed();

	/** Saves selected level under new name */
	UE_API void SaveSelectedLevelAs_Executed();
	
	/** Migrate selected levels */
	UE_API void MigrateSelectedLevels_Executed();

	/** Expand selected items hierarchy */
	UE_API void ExpandSelectedItems_Executed();
			
	/** Check-Out selected levels from SCC */
	UE_API void OnSCCCheckOut();

	/** Mark for Add selected levels from SCC */
	UE_API void OnSCCOpenForAdd();

	/** Check-In selected levels from SCC */
	UE_API void OnSCCCheckIn();

	/** Shows the SCC History of selected levels */
	UE_API void OnSCCHistory();

	/** Refreshes the states selected levels from SCC */
	UE_API void OnSCCRefresh();

	/** Diffs selected levels from with those in the SCC depot */
	UE_API void OnSCCDiffAgainstDepot();

	/** Enable source control features */
	UE_API void OnSCCConnect() const;

	/** Selects all levels in the collection view model */
	UE_API void SelectAllLevels_Executed();

	/** De-selects all levels in the collection view model */
	UE_API void DeselectAllLevels_Executed();

	/** Inverts level selection in the collection view model */
	UE_API void InvertSelection_Executed();

	/** Adds the Actors in the selected Levels from the viewport's existing selection */
	UE_API void SelectActors_Executed();

	/** Removes the Actors in the selected Levels from the viewport's existing selection */
	UE_API void DeselectActors_Executed();

	/** Set level `Use External Actors` to bExternal  */
	UE_API void ConvertLevelToExternalActors_Executed(bool bExternal);

	/** Toggles selected levels to a visible state in the viewports for editor worlds */
	UE_API void ShowInEditorSelectedLevels_Executed();

	/** Toggles selected levels to an invisible state in the viewports for editor worlds */
	UE_API void HideInEditorSelectedLevels_Executed();

	/** Toggles the selected levels to a visible state for editor worlds; toggles all other levels to an invisible state. */
	UE_API void ShowInEditorOnlySelectedLevels_Executed();

	/** Toggles the selected levels to an invisible state for editor worlds; toggles all other levels to a visible state. */
	UE_API void ShowInEditorAllButSelectedLevels_Executed();

	/** Toggles all levels to a visible state in the viewports for editor worlds */
	UE_API void ShowInEditorAllLevels_Executed();

	/** Hides all levels to an invisible state in the viewports for editor worlds */
	UE_API void HideInEditorAllLevels_Executed();

	/** Toggles selected levels to a visible state in the viewports for game worlds */
	UE_API void ShowInGameSelectedLevels_Executed();

	/** Toggles selected levels to an invisible state in the viewports */
	UE_API void HideInGameSelectedLevels_Executed();

	/** Toggles the selected levels to a visible state for game worlds; toggles all other levels to an invisible state. */
	UE_API void ShowInGameOnlySelectedLevels_Executed();

	/** Toggles the selected levels to an invisible state for game worlds; toggles all other levels to a visible state. */
	UE_API void ShowInGameAllButSelectedLevels_Executed();

	/** Toggles all levels to a visible state in the viewports for game worlds */
	UE_API void ShowInGameAllLevels_Executed();

	/** Hides all levels to an invisible state in the viewports for game worlds */
	UE_API void HideInGameAllLevels_Executed();
	
	/** Locks selected levels */
	UE_API void LockSelectedLevels_Executed();

	/** Unlocks selected levels */
	UE_API void UnlockSelectedLevels_Executed();

	/** Toggles the selected levels to a locked state; toggles all other levels to an unlocked state */
	UE_API void LockOnlySelectedLevels_Executed();

	/** Toggles the selected levels to an unlocked state; toggles all other levels to a locked state */
	UE_API void LockAllButSelectedLevels_Executed();

	/** Locks all levels */
	UE_API void LockAllLevels_Executed();

	/** Unlocks all levels */
	UE_API void UnlockAllLevels_Executed();
	
	/** Toggle all read-only levels */
	UE_API void ToggleReadOnlyLevels_Executed();

	/** true if the SCC Check-Out option is available */
	bool CanExecuteSCCCheckOut() const
	{
		return bCanExecuteSCCCheckOut;
	}

	/** true if the SCC Check-In option is available */
	bool CanExecuteSCCCheckIn() const
	{
		return bCanExecuteSCCCheckIn;
	}

	/** true if the SCC Mark for Add option is available */
	bool CanExecuteSCCOpenForAdd() const
	{
		return bCanExecuteSCCOpenForAdd;
	}

	/** true if Source Control options are generally available. */
	bool CanExecuteSCC() const
	{
		return bCanExecuteSCC;
	}
	
	/** Fills MenuBulder with Lock level related commands */
	UE_API void FillLockSubMenu(FMenuBuilder& MenuBuilder);

	/** Fills MenuBulder with level visisbility related commands */
	UE_API void FillEditorVisibilitySubMenu(FMenuBuilder& MenuBuilder);

	/** Fills MenuBulder with level visisbility related commands */
	UE_API void FillGameVisibilitySubMenu(FMenuBuilder& MenuBuilder);

	/** Fills MenuBulder with SCC related commands */
	UE_API void FillSourceControlSubMenu(FMenuBuilder& MenuBuilder);
				
protected:
	/**  */
	UE_API virtual void Initialize(UWorld* InWorld);
	
	/**  */
	UE_API virtual void BindCommands();

	/** Removes the Actors in all read-only Levels from the viewport's existing selection */
	UE_API void DeselectActorsInAllReadOnlyLevel(const FLevelModelList& InLevelList);

	/** Removes the Actors in all read-only Levels from the viewport's existing selection */
	UE_API void DeselectSurfaceInAllReadOnlyLevel(const FLevelModelList& InLevelList);
	
	/** Called whenever level collection has been changed */
	UE_API virtual void OnLevelsCollectionChanged();
	
	/** Called whenever level selection has been changed */
	UE_API virtual void OnLevelsSelectionChanged();

	/** Called whenever level selection has been changed outside of this module, usually via World->SetSelectedLevels */
	UE_API void OnLevelsSelectionChangedOutside();
	
	/** Called whenever level collection hierarchy has been changed */
	UE_API virtual void OnLevelsHierarchyChanged();

	/** Called before loading specified level models into editor */
	virtual void OnPreLoadLevels(const FLevelModelList& InList) {};
	
	/** Called before making visible specified level models */
	virtual void OnPreShowLevels(const FLevelModelList& InList) {};

	/** Called when level was added to the world */
	UE_API void OnLevelAddedToWorld(ULevel* InLevel, UWorld* InWorld);

	/** Called when level was removed from the world */
	UE_API void OnLevelRemovedFromWorld(ULevel* InLevel, UWorld* InWorld);

	/** Handler for FEditorSupportDelegates::RedrawAllViewports event */
	UE_API void OnRedrawAllViewports();

	/** Handler for when an actor was added to a level */
	UE_API void OnLevelActorAdded(AActor* InActor);	

	/** Handler for when an actor was removed from a level */
	UE_API void OnLevelActorDeleted(AActor* InActor);
	
	/** Handler for level filter collection changes */
	UE_API void OnFilterChanged();

	/** Caches the variables for which SCC menu options are available */
	UE_API void CacheCanExecuteSourceControlVars() const;
		
protected:
	
	// The editor world from where we pull our data
	TWeakObjectPtr<UWorld>				CurrentWorld;

	// Has request to update all levels cached 
	bool								bRequestedUpdateAllLevels;
	
	// Has request to redraw all levels
	bool								bRequestedRedrawAllLevels;

	// Has request to update actors count for all levels
	bool								bRequestedUpdateActorsCount;

	/** The list of commands with bound delegates for the Level collection */
	const TSharedRef<FUICommandList>	CommandList;

	/** The collection of filters used to restrict the Levels shown in UI */
	const TSharedRef<LevelFilterCollection> Filters;
	
	/** Levels in the root of hierarchy, persistent levels  */
	FLevelModelList						RootLevelsList;
	
	/** All levels found in the world */
	FLevelModelList						AllLevelsList;
	
	/** All levels in a map<PackageName, LevelModel> */
	TMap<FName, TSharedPtr<FLevelModel>> AllLevelsMap;

	/** Filtered levels from AllLevels list  */
	FLevelModelList						FilteredLevelsList;

	/** Currently selected levels  */
	FLevelModelList						SelectedLevelsList;

	/** Cached value of world size (sum of levels size) */
	FIntPoint							WorldSize;

	/** Whether we should show long package names in level display names */
	bool								bDisplayPaths;

	/** Whether we should show actors count next to level name */
	bool								bDisplayActorsCount;

	/** true if the SCC Check-Out option is available */
	mutable bool						bCanExecuteSCCCheckOut;

	/** true if the SCC Check-In option is available */
	mutable bool						bCanExecuteSCCOpenForAdd;

	/** true if the SCC Mark for Add option is available */
	mutable bool						bCanExecuteSCCCheckIn;

	/** true if Source Control options are generally available. */
	mutable bool						bCanExecuteSCC;

	/** Flag for whether the selection of levels or actors has changed */
	mutable bool						bSelectionHasChanged;

	/** Guard to avoid recursive level selection updates */
	bool								bUpdatingLevelsSelection;
};

//
// Helper struct to temporally make specified UObject immune to dirtying
//
struct FUnmodifiableObject
{
	FUnmodifiableObject(UObject* InObject)
		: ImmuneObject(InObject)
		, bTransient(InObject->HasAnyFlags(RF_Transient))
	{
		if (!bTransient)
		{
			ImmuneObject->SetFlags(RF_Transient);
		}
	}
	
	~FUnmodifiableObject()
	{
		if (!bTransient)
		{
			ImmuneObject->ClearFlags(RF_Transient);
		}
	}

private:
	UObject*		ImmuneObject;
	bool			bTransient;
};

/**  */
struct FTiledLandscapeImportSettings
{
	FTiledLandscapeImportSettings()
		: Scale3D(100.f,100.f,100.f)
		, ComponentsNum(8)
		, QuadsPerSection(63)
		, SectionsPerComponent(1)
		, TilesCoordinatesOffset(0,0)
		, SizeX(1009)
		, bFlipYAxis(true)
	{}
	
	FVector				Scale3D;
	int32				ComponentsNum;
	int32				QuadsPerSection;
	int32				SectionsPerComponent;

	TArray<FString>		HeightmapFileList;
	TArray<FIntPoint>	TileCoordinates;
	FIntPoint			TilesCoordinatesOffset;
	int32				SizeX;
	bool				bFlipYAxis;


	TWeakObjectPtr<UMaterialInterface>	LandscapeMaterial;

	// Landscape layers 
	struct LandscapeLayerSettings
	{
		LandscapeLayerSettings()
			: bNoBlendWeight(false)
		{}

		FName						Name;
		bool						bNoBlendWeight;
		TMap<FIntPoint, FString>	WeightmapFiles;
	};

	TArray<LandscapeLayerSettings>	LandscapeLayerSettingsList;
};

#undef UE_API
