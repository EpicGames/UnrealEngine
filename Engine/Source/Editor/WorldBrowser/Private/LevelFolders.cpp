// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelFolders.h"
#include "Misc/Crc.h"
#include "Misc/Paths.h"
#include "HAL/FileManager.h"
#include "Dom/JsonObject.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "ScopedTransaction.h"
#include "Engine/Level.h"
#include "Editor.h"

#include "IWorldTreeItem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(LevelFolders)

#define LOCTEXT_NAMESPACE "LevelFolders"

/** Utility function to get a hashed filename for a level model */
FString GetLevelModelFilename(TSharedPtr<FLevelModel> LevelModel)
{
	const FString LevelPackage = LevelModel->GetLongPackageName().ToString();
	const uint32 PackageNameCrc = FCrc::MemCrc32(*LevelPackage, sizeof(TCHAR)*LevelPackage.Len());
	return FPaths::Combine(*FPaths::ProjectSavedDir(), TEXT("Config"), TEXT("LevelState"), *FString::Printf(TEXT("%u.json"), PackageNameCrc));
}

static FName OldPathToNewPath(const FString& InOldBranch, const FString& InNewBranch, const FString& PathToMove)
{
	return FName(*(InNewBranch + PathToMove.RightChop(InOldBranch.Len())));
}

FORCEINLINE FLevelModelKey GetLevelModelKey(TSharedRef<FLevelModel> LevelModel)
{
	ULevel* LevelObject = LevelModel->GetLevelObject();
	return LevelObject != nullptr ? *LevelObject->GetPathName() : FName();
}

void UEditorLevelFolders::Serialize(FArchive& Ar)
{
	Ar << Folders;
}

FOnLevelFolderCreate	FLevelFolders::OnFolderCreate;
FOnLevelFolderDelete	FLevelFolders::OnFolderDelete;
FOnLevelFolderMove		FLevelFolders::OnFolderMove;
FLevelFolders*			FLevelFolders::Singleton = nullptr;

FLevelFolders::FLevelFolders()
{
}

FLevelFolders::~FLevelFolders()
{
}

void FLevelFolders::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObjects(TemporaryLevelFolders);
}

FLevelFolders& FLevelFolders::Get()
{
	check(Singleton != nullptr);
	return *Singleton;
}

void FLevelFolders::Init()
{
	Singleton = new FLevelFolders();
}

void FLevelFolders::Cleanup()
{
	delete Singleton;
	Singleton = nullptr;
}

void FLevelFolders::SaveLevel(TSharedRef<FLevelModel> LevelModel)
{
	Housekeeping();

	// Attempt to save the folder state for the levels
	const auto* Folders = TemporaryLevelFolders.Find(GetLevelModelKey(LevelModel));

	if (Folders)
	{
		const FString Filename = GetLevelModelFilename(LevelModel);
		TUniquePtr<FArchive> Ar(IFileManager::Get().CreateFileWriter(*Filename));

		if (Ar)
		{
			TSharedRef<FJsonObject> RootObject = MakeShareable(new FJsonObject);
			TSharedRef<FJsonObject> JsonFolders = MakeShareable(new FJsonObject);

			for (const auto& Pair : (*Folders)->Folders)
			{
				TSharedRef<FJsonObject> JsonFolder = MakeShareable(new FJsonObject);
				JsonFolder->SetBoolField(TEXT("bExpanded"), Pair.Value.bExpanded);

				JsonFolders->SetObjectField(Pair.Key.ToString(), JsonFolder);
			}

			RootObject->SetObjectField(TEXT("Folders"), JsonFolders);
			{
				auto Writer = TJsonWriterFactory<TCHAR>::Create(Ar.Get());
				FJsonSerializer::Serialize(RootObject, Writer);
				Ar->Close();
			}
		}
	}
}

void FLevelFolders::Housekeeping()
{
	for (auto It = TemporaryModelObjects.CreateIterator(); It; ++It)
	{
		// If the level model or its associated level object are invalid, remove them from temporary folders
		if (!It.Value().IsValid())
		{
			TemporaryLevelFolders.Remove(It.Key());
			It.RemoveCurrent();
		}
	}
}

TMap<FName, FLevelFolderProps>& FLevelFolders::GetFolderProperties(TSharedRef<FLevelModel> LevelModel)
{
	return GetOrCreateFoldersForLevel(LevelModel).Folders;
}

FLevelFolderProps* FLevelFolders::GetFolderProperties(TSharedRef<FLevelModel> LevelModel, FName InPath)
{
	return GetFolderProperties(LevelModel).Find(InPath);
}

FName FLevelFolders::GetDefaultFolderName(TSharedRef<FLevelModel> LevelModel, FName ParentPath)
{
	// Gets the folder properties for the world to ensure a unique name
	const TMap<FName, FLevelFolderProps>& Folders = GetFolderProperties(LevelModel);

	// Create a valid base name for the folder
	uint32 NewFolderSuffix = 1;
	FText NewFolderFormat = LOCTEXT("DefaultFolderNamePattern", "NewFolder{0}");

	FString ParentFolderPath;
	if (!ParentPath.IsNone())
	{
		ParentFolderPath = ParentPath.ToString() + TEXT('/');
	}

	FName FolderName;

	do
	{
		FText LeafName = FText::Format(NewFolderFormat, FText::AsNumber(NewFolderSuffix++));
		FolderName = FName(*(ParentFolderPath + LeafName.ToString()));

		if (NewFolderSuffix == 0)
		{
			// If this happens, something's massively broken.
			check(false);
		}
	} while (Folders.Contains(FolderName));

	return FolderName;
}

UEditorLevelFolders& FLevelFolders::GetOrCreateFoldersForLevel(TSharedRef<FLevelModel> LevelModel)
{
	if (auto* Folders = TemporaryLevelFolders.Find(GetLevelModelKey(LevelModel)))
	{
		return **Folders;
	}

	return Initialize(LevelModel);
}

void FLevelFolders::CreateFolder(TSharedRef<FLevelModel> LevelModel, FName InPath)
{
	FScopedTransaction Transaction(LOCTEXT("UndoAction_CreateFolder", "Create Folder"));

	if (AddFolder(LevelModel, InPath))
	{
		OnFolderCreate.Broadcast(LevelModel, InPath);
	}
	else
	{
		Transaction.Cancel();
	}
}

void FLevelFolders::RebuildFolderList(TSharedRef<FLevelModel> LevelModel)
{
	if (TemporaryLevelFolders.Contains(GetLevelModelKey(LevelModel)))
	{
		// Keep empty folders if we rebuild, since they have not been explicitly deleted

		for (TSharedPtr<FLevelModel> Child : LevelModel->GetChildren())
		{
			AddFolder(LevelModel, Child->GetFolderPath());
		}
	}
	else
	{
		Initialize(LevelModel);
	}
}

void FLevelFolders::CreateFolderContainingSelectedLevels(TSharedRef<FLevelCollectionModel> WorldModel, TSharedRef<FLevelModel> LevelModel, FName InPath)
{
	const FScopedTransaction Transaction(LOCTEXT("UndoAction_CreateFolder", "Create Folder"));
	CreateFolder(LevelModel, InPath);
	SetSelectedLevelFolderPath(WorldModel, LevelModel, InPath);
}

void FLevelFolders::SetSelectedLevelFolderPath(TSharedRef<FLevelCollectionModel> WorldModel, TSharedRef<FLevelModel> LevelModel, FName InPath) const
{
	FLevelModelList SelectedLevels = GetSelectedLevels(WorldModel, LevelModel);

	for (TSharedPtr<FLevelModel> Level : SelectedLevels)
	{
		Level->SetFolderPath(InPath);
	}
}

UEditorLevelFolders& FLevelFolders::Initialize(TSharedRef<FLevelModel> LevelModel)
{
	// Clean up stale levels
	Housekeeping();

	// Don't pass RF_Transactional to ConstructObject so that the creation of the object is not recorded by the undo buffer
	UEditorLevelFolders* Folders = NewObject<UEditorLevelFolders>(GetTransientPackage(), NAME_None, RF_NoFlags);
	Folders->SetFlags(RF_Transactional);	// We still want to record changes made to the object, though

	FLevelModelKey LevelModelKey = GetLevelModelKey(LevelModel);

	// Only add the level model if it has a valid key
	if (!LevelModelKey.IsNone())
	{
		TemporaryLevelFolders.Add(LevelModelKey, Folders);
		TemporaryModelObjects.Add(LevelModelKey, LevelModel);

		// Ensure the list is up to date
		for (TSharedPtr<FLevelModel> ChildLevel : LevelModel->GetChildren())
		{
			AddFolder(LevelModel, ChildLevel->GetFolderPath());
		}

		// Attempt to load the folder properties from config
		const FString Filename = GetLevelModelFilename(LevelModel);
		TUniquePtr<FArchive> Ar(IFileManager::Get().CreateFileReader(*Filename));
		if (Ar)
		{
			TSharedPtr<FJsonObject> RootObject = MakeShareable(new FJsonObject);

			auto Reader = TJsonReaderFactory<TCHAR>::Create(Ar.Get());
			if (FJsonSerializer::Deserialize(Reader, RootObject))
			{
				const TSharedPtr<FJsonObject>& JsonFolders = RootObject->GetObjectField(TEXT("Folders"));
				for (const auto& Pair : JsonFolders->Values)
				{
					// Only load properties for folders that still exist in the world
					if (FLevelFolderProps* FolderInWorld = Folders->Folders.Find(FName(*Pair.Key)))
					{
						const TSharedPtr<FJsonObject>& FolderProps = Pair.Value->AsObject();
						FolderInWorld->bExpanded = FolderProps->GetBoolField(TEXT("bExpanded"));
					}
				}
			}
		}
	}

	return *Folders;
}

bool FLevelFolders::AddFolder(TSharedRef<FLevelModel> LevelModel, FName InPath)
{
	bool bFolderAdded = false;

	if (!InPath.IsNone())
	{
		UEditorLevelFolders& LevelFolders = GetOrCreateFoldersForLevel(LevelModel);

		if (!LevelFolders.Folders.Contains(InPath))
		{
			// Also add the parent path
			AddFolder(LevelModel, WorldHierarchy::GetParentPath(InPath));

			LevelFolders.Modify();
			LevelFolders.Folders.Add(InPath);

			bFolderAdded = true;
		}
	}

	return bFolderAdded;
}

bool FLevelFolders::RenameFolder(TSharedRef<FLevelModel> LevelModel, FName OldPath, FName NewPath)
{
	const FString OldPathString = OldPath.ToString();
	const FString NewPathString = NewPath.ToString();

	if (OldPath.IsNone() || NewPath.IsNone() || OldPath == NewPath || PathIsChildOf(NewPathString, OldPathString))
	{
		return false;
	}

	const FScopedTransaction Transaction(LOCTEXT("UndoAction_RenameFolder", "Rename Folder"));

	TSet<FName> RenamedFolders;

	// Move any folders we currently hold
	UEditorLevelFolders& Folders = GetOrCreateFoldersForLevel(LevelModel);
	Folders.Modify();

	auto FoldersCopy = Folders.Folders;
	for (const auto& Pair : FoldersCopy)
	{
		FName Path = Pair.Key;

		const FString FolderPath = Path.ToString();
		if (OldPath == Path || PathIsChildOf(FolderPath, OldPathString))
		{
			const FName NewFolder = OldPathToNewPath(OldPathString, NewPathString, FolderPath);
			if (!Folders.Folders.Contains(NewFolder))
			{
				// Use the existing folder props if we have them
				if (FLevelFolderProps* Props = Folders.Folders.Find(Path))
				{
					Folders.Folders.Add(NewFolder, *Props);
				}
				else
				{
					Folders.Folders.Add(NewFolder);
				}
				OnFolderMove.Broadcast(LevelModel, Path, NewFolder);
				OnFolderCreate.Broadcast(LevelModel, NewFolder);
			}
			RenamedFolders.Add(Path);
		}
	}

	// Delete old folders
	for (const FName& Path : RenamedFolders)
	{
		Folders.Folders.Remove(Path);
		OnFolderDelete.Broadcast(LevelModel, Path);
	}

	return true;
}

void FLevelFolders::DeleteFolder(TSharedRef<FLevelModel> LevelModel, FName FolderToDelete)
{
	const FScopedTransaction Transaction(LOCTEXT("UndoAction_DeleteFolder", "Delete Folder"));

	UEditorLevelFolders& Folders = GetOrCreateFoldersForLevel(LevelModel);
	if (Folders.Folders.Contains(FolderToDelete))
	{
		Folders.Modify();
		Folders.Folders.Remove(FolderToDelete);
		OnFolderDelete.Broadcast(LevelModel, FolderToDelete);
	}
}

bool FLevelFolders::PathIsChildOf(const FString& InPotentialChild, const FString& InParent)
{
	const int32 ParentLen = InParent.Len();
	return InPotentialChild.Len() > ParentLen
		&& (InPotentialChild[ParentLen] == '/' || InPotentialChild[ParentLen] == '\\')
		&& InPotentialChild.Left(ParentLen) == InParent;
}

FLevelModelList FLevelFolders::GetSelectedLevels(TSharedRef<FLevelCollectionModel> WorldModel, TSharedRef<FLevelModel> LevelModel) const
{
	FLevelModelList WorldModelSelectedLevels = WorldModel->GetSelectedLevels();
	FLevelModelList	SelectedLevels;

	for (TSharedPtr<FLevelModel> Model : WorldModelSelectedLevels)
	{
		if (LevelModel->HasDescendant(Model))
		{
			SelectedLevels.Add(Model);
		}
	}

	return SelectedLevels;
}


#undef LOCTEXT_NAMESPACE
