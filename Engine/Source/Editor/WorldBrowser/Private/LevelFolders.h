// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/GCObject.h"

#include "LevelCollectionModel.h"

#include "LevelFolders.generated.h"

#define UE_API WORLDBROWSER_API

typedef		FName		FLevelModelKey;

/** Broadcasted when an editor-only folder has been created for a level */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnLevelFolderCreate, TSharedPtr<FLevelModel>, FName);

/** Broadcasted when an editor-only folder for a level has been deleted  */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnLevelFolderDelete, TSharedPtr<FLevelModel>, FName);

/** Broadcasted when an editor-only folder for a level has moved */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnLevelFolderMove, TSharedPtr<FLevelModel>, FName /* Source */, FName /* Destination */);


/** Properties for level folders */
USTRUCT()
struct FLevelFolderProps
{
	GENERATED_USTRUCT_BODY()

	FLevelFolderProps() : bExpanded(true) {}

	/** Serializer */
	FORCEINLINE friend FArchive& operator<<(FArchive& Ar, FLevelFolderProps& Folder)
	{
		return Ar << Folder.bExpanded;
	}

	bool bExpanded;
};


/** Level folder UObject, for supporting undo/redo functionality */
UCLASS()
class UEditorLevelFolders : public UObject
{
	GENERATED_BODY()

public:
	virtual void Serialize(FArchive& Ar) override;

	TMap<FName, FLevelFolderProps> Folders;
};


/** The class for managing in-memory representations of level folders in the editor */
struct FLevelFolders : public FGCObject
{
	UE_API FLevelFolders();
	UE_API ~FLevelFolders();

public:

	//~ FGCObject
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const
	{
		return "FLevelFolders";
	}

public:

	/** Checks if the singleton is valid */
	static bool IsAvailable() { return Singleton != nullptr; }

	/** Grants access to the singleton object if it's available */
	static UE_API FLevelFolders& Get();

	/** Initialize the singleton */
	static UE_API void Init();

	/** Clean up the singleton */
	static UE_API void Cleanup();


	/** Folder events */
	static UE_API FOnLevelFolderCreate OnFolderCreate;
	static UE_API FOnLevelFolderDelete OnFolderDelete;
	static UE_API FOnLevelFolderMove OnFolderMove;

	/** Gets all folder properties for a specified level */
	UE_API TMap<FName, FLevelFolderProps>& GetFolderProperties(TSharedRef<FLevelModel> LevelModel);

	/** Gets the folder properties for a specified path within the level */
	UE_API FLevelFolderProps* GetFolderProperties(TSharedRef<FLevelModel> LevelModel, FName InPath);

	/** Gets the default folder name for the given path. */
	UE_API FName GetDefaultFolderName(TSharedRef<FLevelModel> LevelModel, FName ParentPath);

	/** Creates a new folder with the given name for the current level selection */
	UE_API void CreateFolderContainingSelectedLevels(TSharedRef<FLevelCollectionModel> WorldModel, TSharedRef<FLevelModel> LevelModel, FName InPath);

	/** Creates a folder for the level model with the given path name */
	UE_API void CreateFolder(TSharedRef<FLevelModel> LevelModel, FName InPath);

	/** Renames a folder. The folder with the old name is removed from the folder props */
	UE_API bool RenameFolder(TSharedRef<FLevelModel> LevelModel, FName OldPath, FName NewPath);

	/** Deletes a folder and all saved properties */
	UE_API void DeleteFolder(TSharedRef<FLevelModel> LevelModel, FName FolderToDelete);

	/** Rebuilds the folder list for the level */
	UE_API void RebuildFolderList(TSharedRef<FLevelModel> LevelModel);

	/** Saves the level model when the world is saved */
	UE_API void SaveLevel(TSharedRef<FLevelModel> LevelModel);

private:

	/** Sets the folder path for the current level selection */
	UE_API void SetSelectedLevelFolderPath(TSharedRef<FLevelCollectionModel> WorldModel, TSharedRef<FLevelModel> LevelModel, FName InPath) const;

	/** Remove references to folder arrays for unloaded levels */
	UE_API void Housekeeping();

	/** Create a folder container for the specified level model */
	UE_API UEditorLevelFolders& InitializeForLevel(TSharedRef<FLevelModel> LevelModel);

	/** Checks if the supplied path is a descendant of the parent path */
	UE_API bool PathIsChildOf(const FString& InPotentialChild, const FString& InParent);

	/** Gets the folder information for the given level, or creates it if it's not in memory */
	UE_API UEditorLevelFolders& GetOrCreateFoldersForLevel(TSharedRef<FLevelModel> LevelModel);

	/** Creates new folder information for the level */
	UE_API UEditorLevelFolders& Initialize(TSharedRef<FLevelModel> LevelModel);

	/** Adds a folder for the level without triggering any events */
	UE_API bool AddFolder(TSharedRef<FLevelModel> LevelModel, FName InPath);

	/** Gets the selected levels in the world model */
	UE_API FLevelModelList GetSelectedLevels(TSharedRef<FLevelCollectionModel> WorldModel, TSharedRef<FLevelModel> LevelModel) const;

private:

	/** Transient map of folders, keyed off level path name  */
	TMap<FLevelModelKey /*Level Path Name*/, TObjectPtr<UEditorLevelFolders>> TemporaryLevelFolders;

	/** Maps level paths to level model objects, to clear out unloaded level model information when necessary */
	TMap<FLevelModelKey, TWeakPtr<FLevelModel>> TemporaryModelObjects;

	/** Singleton object to be maintained by the world browser module */
	static UE_API FLevelFolders* Singleton;
};

#undef UE_API
