// Copyright Epic Games, Inc. All Rights Reserved.
#include "LevelModel.h"
#include "GameFramework/Actor.h"
#include "Misc/MessageDialog.h"
#include "HAL/FileManager.h"
#include "Model.h"
#include "Modules/ModuleManager.h"
#include "Misc/PackageName.h"
#include "Engine/Brush.h"
#include "GameFramework/WorldSettings.h"
#include "Engine/Selection.h"
#include "Editor.h"
#include "ScopedTransaction.h"
#include "LevelUtils.h"
#include "EditorLevelUtils.h"
#include "ActorEditorUtils.h"
#include "LevelEditor.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Elements/Framework/TypedElementCommonActions.h"
#include "Elements/Interfaces/TypedElementDetailsInterface.h"

#include "Engine/LevelScriptBlueprint.h"

#include "LevelCollectionModel.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Subsystems/AssetEditorSubsystem.h"

#define LOCTEXT_NAMESPACE "WorldBrowser"

FLevelModel::FLevelModel(FLevelCollectionModel& InLevelCollectionModel)
	: LevelCollectionModel(InLevelCollectionModel)
	, bSelected(false)
	, bExpanded(false)
	, bLoadingLevel(false)
	, bFilteredOut(false)
	, LevelTranslationDelta(0,0)
	, LevelActorsCount(0)
{
	SimulationStatus = FSimulationLevelStatus();

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	AssetRegistryModule.Get().OnAssetRenamed().AddRaw(this, &FLevelModel::OnAssetRenamed);
}

FLevelModel::~FLevelModel()
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	AssetRegistryModule.Get().OnAssetRenamed().RemoveAll(this);
}

void FLevelModel::OnAssetRenamed(const FAssetData& AssetData, const FString& OldObjectPath)
{
	const FString CurrentPackage = GetLongPackageName().ToString();

	if (FPackageName::ObjectPathToPackageName(OldObjectPath) == CurrentPackage)
	{
		UpdateAsset(AssetData);
		UpdateDisplayName();
	}
}

void FLevelModel::SetLevelSelectionFlag(bool bSelectedFlag)
{
	bSelected = bSelectedFlag;
}

bool FLevelModel::GetLevelSelectionFlag() const
{
	return bSelected;
}

void FLevelModel::SetLevelExpansionFlag(bool bExpandedFlag)
{
	bExpanded = bExpandedFlag;
}

bool FLevelModel::GetLevelExpansionFlag() const
{
	return bExpanded;
}

void FLevelModel::SetLevelFilteredOutFlag(bool bFiltredOutFlag)
{
	bFilteredOut = bFiltredOutFlag;
}
	
bool FLevelModel::GetLevelFilteredOutFlag() const
{
	return bFilteredOut;
}

FString FLevelModel::GetDisplayName() const
{
	return DisplayName;
}

FString FLevelModel::GetPackageFileName() const
{
	const FName LocalPackageName = GetLongPackageName();
	if (LocalPackageName != NAME_None)
	{
		return FPackageName::LongPackageNameToFilename(LocalPackageName.ToString(), FPackageName::GetMapPackageExtension());
	}
	else
	{
		return FString();
	}
}

void FLevelModel::Accept(FLevelModelVisitor& Vistor)
{
	Vistor.Visit(*this);
	
	for (auto It = AllChildren.CreateIterator(); It; ++It)
	{
		(*It)->Accept(Vistor);
	}
}

void FLevelModel::Update()
{
	UpdateLevelActorsCount();
	BroadcastChangedEvent();
}
	
void FLevelModel::UpdateVisuals()
{
	BroadcastChangedEvent();
}

bool FLevelModel::IsSimulating() const
{
	return LevelCollectionModel.IsSimulating();
}

bool FLevelModel::IsCurrent() const
{
	if (GetLevelObject())
	{
		return GetLevelObject()->IsCurrentLevel();
	}

	return false;
}

bool FLevelModel::IsPersistent() const
{
	return LevelCollectionModel.GetWorld()->PersistentLevel == GetLevelObject();
}

bool FLevelModel::IsEditable() const
{
	return (IsLoaded() == true && IsLocked() == false);
}

bool FLevelModel::IsDirty() const
{
	if (GetLevelObject())
	{
		return GetLevelObject()->GetOutermost()->IsDirty();
	}
	
	return false;
}

bool FLevelModel::IsLightingScenario() const
{
	if (GetLevelObject())
	{
		return GetLevelObject()->bIsLightingScenario;
	}
	
	return false;
}

void FLevelModel::SetIsLightingScenario(bool bNew)
{
	if (GetLevelObject())
	{
		GetLevelObject()->SetLightingScenario(bNew);
	}
}

bool FLevelModel::IsLoaded() const
{
	return (LevelCollectionModel.IsSimulating() ? SimulationStatus.bLoaded : (GetLevelObject() != NULL));
}

bool FLevelModel::IsLoading() const
{
	return (LevelCollectionModel.IsSimulating() ? SimulationStatus.bLoading : bLoadingLevel);
}

bool FLevelModel::IsVisibleInEditor() const
{
	if (LevelCollectionModel.IsSimulating())
	{
		return SimulationStatus.bVisible;
	}
	else
	{
		ULevel* Level = GetLevelObject();
		if (Level)
		{
			if (ULevelStreaming* StreamingLevel = FLevelUtils::FindStreamingLevel(Level))
			{
				return StreamingLevel->ShouldBeVisible();
			}
			else
			{
				return FLevelUtils::IsLevelVisible(Level);
			}
		}
		return false;
	}
}

bool FLevelModel::IsVisibleInGame() const
{
	ULevel* Level = GetLevelObject();
	if (!Level)
	{
		return false;
	}

	// Unreal does not allow the persistent level to be invisible in game.
	if (Level->IsPersistentLevel())
	{
		return true;
	}
	
	ULevelStreaming* StreamingLevel = FLevelUtils::FindStreamingLevel(Level);
	return StreamingLevel && StreamingLevel->GetShouldBeVisibleFlag();
}

bool FLevelModel::IsLocked() const
{
	ULevel* Level = GetLevelObject();
	if (Level)
	{
		return FLevelUtils::IsLevelLocked(Level);
	}

	return false;	
}

bool FLevelModel::IsFileReadOnly() const
{
	if (HasValidPackage())
	{
		FName PackageName = GetLongPackageName();
		
		FString PackageFileName;
		if (FPackageName::DoesPackageExist(PackageName.ToString(), &PackageFileName))
		{
			return IFileManager::Get().IsReadOnly(*PackageFileName);
		}
	}

	return false;
}

bool FLevelModel::IsUserManaged() const
{
	ULevel* Level = GetLevelObject();
	if (Level)
	{
		if (ULevelStreaming* StreamingLevel = FLevelUtils::FindStreamingLevel(Level))
		{
			return StreamingLevel->IsUserManaged();
		}
	}

	return true;
}

void FLevelModel::LoadLevel()
{

}

void FLevelModel::SetVisibleInEditor(bool bVisible)
{
	TArray<FLevelModel*> LevelModels({ this });
	TArray<bool> bAreVisible({ bVisible });
	FLevelModel::SetVisibleInEditor(LevelModels, bAreVisible);
}

void FLevelModel::SetVisibleInEditor(TArray<FLevelModel*>& LevelModels, const TArray<bool>& bAreVisible)
{
	// Don't create unnecessary transactions
	if (LevelModels.Num() == 0 || LevelModels.Num() != bAreVisible.Num())
	{
		return;
	}

	TArray<ULevel*> Levels;
	TArray<bool> bAreVisibleCleaned; // Required because of the continue in the for loop
	TArray<bool> bOldAreDirty;
	for (int32 LevelModelIdx = 0; LevelModelIdx < LevelModels.Num(); ++LevelModelIdx)
	{
		FLevelModel* LevelModel = LevelModels[LevelModelIdx];
		// If visibility does not change, omit it
		if (LevelModel->IsVisibleInEditor() == bAreVisible[LevelModelIdx])
		{
			continue;
		}
		// Otherwise, add Level and IsDirty into each TArray
		Levels.Add(LevelModel->GetLevelObject());
		bAreVisibleCleaned.Add(bAreVisible[LevelModelIdx]);
		bOldAreDirty.Add(LevelModel->IsDirty());
	}
	// Don't create unnecessary transactions
	if (Levels.Num() == 0)
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("ToggleVisibility", "Toggle Level Visibility"));

	// This call hides all owned actors, etc.
	EditorLevelUtils::SetLevelsVisibility(Levels, bAreVisibleCleaned, false);

	// Note that Levels.Num() != LevelModels.Num() given the continue in the first for loop
	for (int32 LevelModelIdx = 0; LevelModelIdx < Levels.Num(); ++LevelModelIdx)
	{
		if (!bOldAreDirty[LevelModelIdx])
		{
			// Don't set the dirty flag if we're just changing the visibility of the level within the editor
			ULevel* Level = Levels[LevelModelIdx];
			if (Level)
			{
				Level->GetOutermost()->SetDirtyFlag(false);
			}
		}
	}
}

void FLevelModel::SetVisibleInGame(TArray<FLevelModel*>& LevelModels, const TArray<bool>& bAreVisible)
{
	FScopedTransaction Transaction(LOCTEXT("ToggleGameVisibility", "Toggle Game Visibility"));
	bool bMadeChange = false;
	
	for (int32 i = 0; i < LevelModels.Num(); ++i)
	{
		ULevel* Level = LevelModels[i]->GetLevelObject();
		ULevelStreaming* LevelStreaming = FLevelUtils::FindStreamingLevel(Level);
		const bool bShouldBeVisible = bAreVisible[i];
		if (LevelStreaming && LevelStreaming->GetShouldBeVisibleFlag() != bShouldBeVisible)
		{
			bMadeChange = true;
			Level->Modify();
			LevelStreaming->Modify();
			LevelStreaming->SetShouldBeVisible(bShouldBeVisible);
		}
	}

	if (!bMadeChange)
	{
		Transaction.Cancel();
	}
}

void FLevelModel::SetLocked(bool bLocked)
{
	if (LevelCollectionModel.IsReadOnly())
	{
		return;
	}
		
	ULevel* Level = GetLevelObject();

	if (Level == NULL)
	{
		return;
	}

	// Do nothing if attempting to set the level to the same locked state
	if (bLocked == IsLocked())
	{
		return;
	}

	// If locking the level, deselect all of its actors and BSP surfaces
	if (bLocked)
	{
		DeselectAllSurfaces();

		FLevelEditorModule& LevelEditorModule = FModuleManager::Get().LoadModuleChecked<FLevelEditorModule>("LevelEditor");
		TSharedPtr<ILevelEditor> Editor = LevelEditorModule.GetFirstLevelEditor();
		UTypedElementSelectionSet* SelectionSet = Editor->GetMutableElementSelectionSet();
		TArray<FTypedElementHandle> LevelElementHandles;

		LevelElementHandles.Reserve(Level->Actors.Num());

		// filter out elements that don't belong to the current level
		SelectionSet->ForEachSelectedElement<ITypedElementWorldInterface>(
			[Level, &LevelElementHandles](const TTypedElement<ITypedElementWorldInterface>& Element)
			{
				if (Element.GetOwnerLevel() == Level)
				{
					LevelElementHandles.Add(Element);
				}
				return true;
			});

		// deselect all the elements in the current level
		SelectionSet->DeselectElements(LevelElementHandles, FTypedElementSelectionOptions());

		// Tell the editor selection status was changed.
		GEditor->NoteSelectionChange();

		// If locking the current level, reset the p-level as the current level
		//@todo: fix this!
	}

	// Change the level's locked status
	FLevelUtils::ToggleLevelLock(Level);
}

void FLevelModel::MakeLevelCurrent()
{
	if (LevelCollectionModel.IsReadOnly())
	{
		return;
	}

	if (!IsLoaded())
	{
		// Load level from disk
		FLevelModelList LevelsList; LevelsList.Add(this->AsShared());
		LevelCollectionModel.LoadLevels(LevelsList);
	}

	ULevel* Level = GetLevelObject();
	if (Level == NULL)
	{
		return;
	}
		
	// Locked levels can't be made current.
	if (!FLevelUtils::IsLevelLocked(Level))
	{ 
		// Make current.
		if (LevelCollectionModel.GetWorld()->SetCurrentLevel(Level))
		{
			FEditorDelegates::NewCurrentLevel.Broadcast();
				
			// Deselect all selected builder brushes.
			bool bDeselectedSomething = false;
			for (FSelectionIterator It(GEditor->GetSelectedActorIterator()); It; ++It)
			{
				AActor* Actor = static_cast<AActor*>(*It);
				checkSlow(Actor->IsA(AActor::StaticClass()));

				ABrush* Brush = Cast< ABrush >( Actor );
				if (Brush && FActorEditorUtils::IsABuilderBrush(Brush))
				{
					GEditor->SelectActor(Actor, /*bInSelected=*/ false, /*bNotify=*/ false);
					bDeselectedSomething = true;
				}
			}

			// Send a selection change callback if necessary.
			if (bDeselectedSomething)
			{
				GEditor->NoteSelectionChange();
			}
		}
							
		// Force the current level to be visible.
		LevelCollectionModel.ShowLevelsInEditor({ AsShared() });
	}
	else
	{
		FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "Error_OperationDisallowedOnLockedLevelMakeLevelCurrent", "MakeLevelCurrent: The requested operation could not be completed because the level is locked."));
	}

	Update();
}

bool FLevelModel::HitTest2D(const FVector2D& Point) const
{
	return false;
}
	
FVector2D FLevelModel::GetLevelPosition2D() const
{
	return FVector2D::ZeroVector;
}

FVector2D FLevelModel::GetLevelSize2D() const
{
	return FVector2D::ZeroVector;
}

FBox FLevelModel::GetLevelBounds() const
{
	return FBox(ForceInit);
}

FVector2D FLevelModel::GetLevelTranslationDelta() const
{
	return LevelTranslationDelta;
}

void FLevelModel::SetLevelTranslationDelta(FVector2D InAbsoluteDelta)
{
	LevelTranslationDelta = InAbsoluteDelta;
	
	for (auto It = AllChildren.CreateIterator(); It; ++It)
	{
		(*It)->SetLevelTranslationDelta(InAbsoluteDelta);
	}
}

FLinearColor FLevelModel::GetLevelColor() const
{
	// Returns Constant color, base classes will override this
	// Currently not all base classes have the requisite support, so I've not made it pure virtual.
	return FLinearColor::White;
}

void FLevelModel::SetLevelColor(FLinearColor InColor)
{
	// Does nothing, base classes will override this
	// Currently not all base classes have the requisite support, so I've not made it pure virtual.
}

bool FLevelModel::IsVisibleInCompositionView() const
{
	return false;
}

bool FLevelModel::HasKismet() const
{
	return (GetLevelObject() != NULL);
}

void FLevelModel::OpenKismet()
{
	if (LevelCollectionModel.IsReadOnly())
	{
		return;
	}
	
	ULevel* Level = GetLevelObject();

	if (Level == NULL)
	{
		return;
	}
	
	ULevelScriptBlueprint* LevelScriptBlueprint = Level->GetLevelScriptBlueprint();
	if (LevelScriptBlueprint)
	{
		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(LevelScriptBlueprint);
	}
	else
	{
		FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "UnableToCreateLevelScript", "Unable to find or create a level blueprint for this level.") );
	}
}

bool FLevelModel::AttachTo(TSharedPtr<FLevelModel> InParent)
{
	if (LevelCollectionModel.IsReadOnly() || 
		!IsLoaded() || 
		IsPersistent() ||
		InParent.IsValid() == false ||
		InParent.Get() == this ||
		HasDescendant(InParent))
	{
		return false;
	}

	TSharedPtr<FLevelModel> CurrentParent = GetParent();
	if (CurrentParent.IsValid())
	{
		CurrentParent->RemoveChild(this->AsShared());
	}

	Parent = InParent;

	CurrentParent = GetParent();
	if (CurrentParent.IsValid())
	{
		CurrentParent->AddChild(this->AsShared());
	}

	OnParentChanged();
	return true;
}

void FLevelModel::OnFilterChanged()
{
	FilteredChildren.Empty();
	
	for (const auto& LevelModel : AllChildren)
	{
		LevelModel->OnFilterChanged();

		// Item will pass filtering regardless of filter settings if it has children that passes filtering
		if (LevelModel->GetChildren().Num() > 0 || LevelCollectionModel.PassesAllFilters(*LevelModel))
		{
			FilteredChildren.Add(LevelModel);
		}
	}
}

const FLevelModelList& FLevelModel::GetChildren() const
{
	return FilteredChildren;
}
	
TSharedPtr<FLevelModel> FLevelModel::GetParent() const
{
	return Parent.Pin();
}

void FLevelModel::SetParent(TSharedPtr<FLevelModel> InParent)
{
	Parent = InParent;
}

void FLevelModel::RemoveAllChildren()
{
	FilteredChildren.Empty();
	AllChildren.Empty();
}

void FLevelModel::RemoveChild(TSharedPtr<FLevelModel> InChild)
{
	FilteredChildren.Remove(InChild);
	AllChildren.Remove(InChild);
}
	
void FLevelModel::AddChild(TSharedPtr<FLevelModel> InChild)
{
	AllChildren.AddUnique(InChild);

	if (LevelCollectionModel.PassesAllFilters(*InChild))
	{
		FilteredChildren.Add(InChild);
	}
}

bool FLevelModel::HasAncestor(const TSharedPtr<FLevelModel>& InLevel) const
{
	TSharedPtr<FLevelModel> ParentModel = GetParent();
	while (ParentModel.IsValid())
	{
		if (ParentModel == InLevel)
		{
			return true;
		}
		
		ParentModel = ParentModel->GetParent();
	}
	
	return false;
}

bool FLevelModel::HasDescendant(const TSharedPtr<FLevelModel>& InLevel) const
{
	if (AllChildren.Find(InLevel) != INDEX_NONE)
	{
		return true;
	}
	
	for (auto It = AllChildren.CreateConstIterator(); It; ++It)
	{
		if ((*It)->HasDescendant(InLevel))
		{
			return true;
		}
	}
	
	return false;
}

void FLevelModel::OnDrop(const TSharedPtr<FLevelDragDropOp>& Op)
{
}
	
bool FLevelModel::IsGoodToDrop(const TSharedPtr<FLevelDragDropOp>& Op) const
{
	return false;
}

void FLevelModel::OnLevelAddedToWorld(ULevel* InLevel)
{
	UpdateLevelActorsCount();
}

void FLevelModel::OnLevelRemovedFromWorld()
{
	UpdateLevelActorsCount();
}

void FLevelModel::BroadcastChangedEvent()
{
	ChangedEvent.Broadcast();
}

void FLevelModel::UpdateSimulationStatus(ULevelStreaming* StreamingLevel)
{
	SimulationStatus = FSimulationLevelStatus();
	
	// Persistent level always loaded and visible in PIE
	if (IsPersistent())
	{
		SimulationStatus.bLoaded = true;
		SimulationStatus.bVisible = true;
		return;
	}

	if (StreamingLevel == nullptr)
	{
		return;
	}
		
	if (StreamingLevel->GetLoadedLevel())
	{
		SimulationStatus.bLoaded = true;
				
		if (StreamingLevel->GetLoadedLevel()->bIsVisible)
		{
			SimulationStatus.bVisible = true;
		}
	}
	else if (StreamingLevel->HasLoadRequestPending())
	{
		SimulationStatus.bLoading = true;
	}
}

void FLevelModel::DeselectAllSurfaces()
{
	ULevel* Level = GetLevelObject();

	if (Level == NULL)
	{
		return;
	}

	UModel* Model = Level->Model;
	for (int32 SurfaceIndex = 0; SurfaceIndex < Model->Surfs.Num(); ++SurfaceIndex)
	{
		FBspSurf& Surf = Model->Surfs[SurfaceIndex];
		if (Surf.PolyFlags & PF_Selected)
		{
			Model->ModifySurf(SurfaceIndex, false);
			Surf.PolyFlags&= ~PF_Selected;
		}
	}
}

void FLevelModel::DeselectAllActors()
{
	ULevel* Level = GetLevelObject();

	if (Level == NULL)
	{
		return;
	}

	USelection* SelectedActors = GEditor->GetSelectedActors();
	SelectedActors->Modify();

	// Deselect all level actors 
	for (auto It = Level->Actors.CreateIterator(); It; ++It)
	{
		AActor* CurActor = (*It);
		if (CurActor)
		{
			GEditor->SelectActor(CurActor, false, false);
		}
	}
}

void FLevelModel::SelectActors(bool bSelect, bool bNotify, bool bSelectEvenIfHidden,
							   const TSharedPtr<ActorFilter>& Filter)
{
	if (LevelCollectionModel.IsReadOnly())
	{
		return;
	}
	
	ULevel* Level = GetLevelObject();

	if (Level == NULL || IsLocked())
	{
		return;
	}

	GEditor->GetSelectedActors()->BeginBatchSelectOperation();
	bool bChangesOccurred = false;

	// Iterate over all actors, looking for actors in this level.
	for (auto It = Level->Actors.CreateIterator(); It; ++It)
	{
		AActor* Actor = (*It);
		if (Actor)
		{
			if (Filter.IsValid() && !Filter->PassesFilter(Actor))
			{
				continue;
			}
			
			//exclude the world settings and builder brush from actors selected
			const bool bIsWorldSettings = Actor->IsA(AWorldSettings::StaticClass());
			const bool bIsBuilderBrush = (Actor->IsA(ABrush::StaticClass()) && FActorEditorUtils::IsABuilderBrush(Actor));
			if (bIsWorldSettings || bIsBuilderBrush)
			{
				continue;
			}

			bool bNotifyForActor = false;
			GEditor->GetSelectedActors()->Modify();
			GEditor->SelectActor(Actor, bSelect, bNotifyForActor, bSelectEvenIfHidden);
			bChangesOccurred = true;
		}
	}

	GEditor->GetSelectedActors()->EndBatchSelectOperation();

	if (bNotify)
	{
		GEditor->NoteSelectionChange();
	}
}

void FLevelModel::ConvertLevelToExternalActors(bool bUseExternal)
{
	ULevel* Level = GetLevelObject();
	if (Level == nullptr || IsLocked() || !IsUserManaged())
	{
		return;
	}
	Level->Modify();
	Level->ConvertAllActorsToPackaging(bUseExternal);
}

bool FLevelModel::CanConvertLevelToExternalActors(bool bToExternal)
{
	ULevel* Level = GetLevelObject();
	if (Level == nullptr || IsLocked() || !IsUserManaged())
	{
		return false;
	}

	UPackage* LevelPackage = Level->GetOutermost();
	if (Level->IsUsingExternalActors() == bToExternal
		|| LevelPackage == GetTransientPackage()
		|| LevelPackage->HasAnyFlags(RF_Transient)
		|| Level->HasAnyFlags(RF_Transient)
		|| !FPackageName::IsValidLongPackageName(LevelPackage->GetName()))
	{
		return false;
	}
	return true;
}

void FLevelModel::UpdateLevelActorsCount()
{
	LevelActorsCount = 0;
	ULevel* Level = GetLevelObject();
	
	if (Level)
	{
		// Count the actors contained in these levels
		// NOTE: We subtract two here to omit "default actors" in the count (default brush, and WorldSettings)
		LevelActorsCount = Level->Actors.Num()-2;

		// Count deleted actors
		int32 NumDeletedActors = 0;
		for (int32 ActorIdx = 0; ActorIdx < Level->Actors.Num(); ++ActorIdx)
		{
			if (!Level->Actors[ActorIdx])
			{
				++NumDeletedActors;
			}
		}
		
		// Subtract deleted actors from the actor count
		LevelActorsCount -= NumDeletedActors;
	}

	UpdateDisplayName();
}

void FLevelModel::UpdateDisplayName()
{
	if (IsPersistent())
	{
		DisplayName = LOCTEXT("PersistentTag", "Persistent Level").ToString();
	}
	else
	{
		DisplayName = GetLongPackageName().ToString();
		if (!LevelCollectionModel.GetDisplayPathsState())
		{
			DisplayName = FPackageName::GetShortName(DisplayName);
		}
	}

	if (HasValidPackage())
	{
		// Append actors count
		if (LevelCollectionModel.GetDisplayActorsCountState() && IsLoaded())
		{
			DisplayName += TEXT(" (");
			DisplayName.AppendInt(LevelActorsCount);
			DisplayName += TEXT(")");
		}
	}
	else
	{
		DisplayName+= LOCTEXT("MissingLevelErrorText", " [Missing Level] ").ToString();
	}
}

FString FLevelModel::GetLightmassSizeString() const
{
	FString MemorySizeString;
	ULevel* Level = GetLevelObject();

	//if (Level && GetDefault<ULevelBrowserSettings>()->bDisplayLightmassSize)
	//{
	//	// Update metrics
	//	static const float ByteConversion = 1.0f / 1024.0f;
	//	float LightmapSize = Level->LightmapTotalSize * ByteConversion;
	//	
	//	MemorySizeString += FString::Printf(TEXT( "%.2f" ), LightmapSize);
	//}

	return MemorySizeString;
}

FString FLevelModel::GetFileSizeString() const
{
	FString MemorySizeString;
	ULevel* Level = GetLevelObject();

	//if (Level && GetDefault<ULevelBrowserSettings>()->bDisplayFileSize)
	//{
	//	// Update metrics
	//	static const float ByteConversion = 1.0f / 1024.0f;
	//	float FileSize = Level->GetOutermost()->GetFileSize() * ByteConversion * ByteConversion;
	//	
	//	MemorySizeString += FString::Printf(TEXT( "%.2f" ), FileSize);
	//}

	return MemorySizeString;
}

UClass* FLevelModel::GetStreamingClass() const
{
	return nullptr;
}

#undef LOCTEXT_NAMESPACE
