// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#define UE_API WORLDBROWSER_API

class AActor;
struct FAssetData;
class FLevelCollectionModel;
class FLevelDragDropOp;
class FLevelModel;
class ULevel;
class ULevelStreaming;
template< typename TItemType > class IFilter;

typedef TArray<TSharedPtr<class FLevelModel>> FLevelModelList;
class FLevelCollectionModel;

/**
 *  Interface for level collection hierarchy traversal
 */
struct FLevelModelVisitor
{
	virtual ~FLevelModelVisitor() = default;
	virtual void Visit(FLevelModel& Item) = 0;
};

/**
 * Interface for non-UI presentation logic for a level in a world
 */
class FLevelModel
	: public TSharedFromThis<FLevelModel>	
{
public:
	typedef IFilter<const TWeakObjectPtr<AActor>&> ActorFilter;

	DECLARE_EVENT( FLevelModel, FSimpleEvent );
	
public:
	UE_API FLevelModel(FLevelCollectionModel& InLevelCollectionModel);

	UE_API virtual ~FLevelModel();
	
	/** Traverses level model hierarchy */
	UE_API void Accept(FLevelModelVisitor& Vistor);

	/** Sets level selection flag */
	UE_API void SetLevelSelectionFlag(bool bExpanded);
	
	/** @return Level selection flag */
	UE_API bool GetLevelSelectionFlag() const;
		
	/** Sets level child hierarchy expansion flag */
	UE_API void SetLevelExpansionFlag(bool bExpanded);
	
	/** @return Level child hierarchy expansion flag */
	UE_API bool GetLevelExpansionFlag() const;

	/** Sets level filtered out flag */
	UE_API void SetLevelFilteredOutFlag(bool bFiltredOut);
	
	/** @return Whether this level model was filtered out */
	UE_API bool GetLevelFilteredOutFlag() const;

	/**	@return	Level display name */
	UE_API FString GetDisplayName() const;
	
	/**	@return	Level package file name */
	UE_API FString GetPackageFileName() const;

	/**	@return	Whether level model has valid package file */
	virtual bool HasValidPackage() const { return true; };
		
	/** @return Pointer to UObject to be used as key in SNodePanel */
	virtual UObject* GetNodeObject() = 0;

	/** @return ULevel object if any */
	virtual ULevel* GetLevelObject() const = 0;
		
	/**	@return	Level asset name */
	virtual FName GetAssetName() const = 0;

	/**	@return	Level package file name */
	virtual FName GetLongPackageName() const = 0;

	/** Update asset associated with level model */
	virtual void UpdateAsset(const FAssetData& AssetData) = 0;
	
	/** Refreshes cached data */
	UE_API virtual void Update();
	
	/** Refreshes visual information */
	UE_API virtual void UpdateVisuals();

	/**	@return	Whether level is in PIE/SIE mode */
	UE_API bool IsSimulating() const;
	
	/** @return Whether level is CurrentLevel */
	UE_API bool IsCurrent() const;

	/** @return Whether level is PersistentLevel */
	UE_API bool IsPersistent() const;

	/** @return Whether level is editable */
	UE_API bool IsEditable() const;

	/** @return Whether level is dirty */
	UE_API bool IsDirty() const;

	/** @return Whether level is a lighting scenario */
	UE_API bool IsLightingScenario() const;

	UE_API void SetIsLightingScenario(bool bNew);

	/** @return Whether level has loaded content */
	UE_API bool IsLoaded() const;

	/** @return Whether level is in process of loading content */
	UE_API bool IsLoading() const;

	/**	@return Whether level is visible in editor worlds */
	UE_API bool IsVisibleInEditor() const;

	/**	@return Whether level is visible in game worlds */
	UE_API bool IsVisibleInGame() const;

	/**	@return Whether level is locked */
	UE_API bool IsLocked() const;

	/** @return Whether level package file is read only */
	UE_API bool IsFileReadOnly() const;

	/** @return Whether this level can be add/removed/unloaded by user */
	UE_API bool IsUserManaged() const;

	/** @return Whether the streaming level object is transient */
	virtual bool IsTransient() const { return false; }

	/** Loads level into editor */
	UE_API virtual void LoadLevel();

	/** Sets the Level's visibility */
	UE_API virtual void SetVisibleInEditor(bool bVisible);

	/** Static function analog to the non-static SetVisible but applied to an TArray of FLevelModel elements */
	static UE_API void SetVisibleInEditor(TArray<FLevelModel*>& LevelModels, const TArray<bool>& bAreVisible);

	static UE_API void SetVisibleInGame(TArray<FLevelModel*>& LevelModels, const TArray<bool>& bAreVisible);

	/** Sets the Level's locked/unlocked state */
	UE_API void SetLocked(bool bLocked);

	/** Sets Level as current in the world */
	UE_API void MakeLevelCurrent();
	
	/** @return Whether specified point is hovering level */
	UE_API virtual bool HitTest2D(const FVector2D& Point) const;
	
	/** @return Level top left corner position */
	UE_API virtual FVector2D GetLevelPosition2D() const;

	/** @return XY size of level */
	UE_API virtual FVector2D GetLevelSize2D() const;

	/** @return Level bounding box */
	UE_API virtual FBox GetLevelBounds() const;
	
	/** @return level translation delta, when user moving level item */
	UE_API FVector2D GetLevelTranslationDelta() const;

	/** Sets new translation delta to this model and all descendants*/
	UE_API void SetLevelTranslationDelta(FVector2D InAbsoluteDelta);

	/** @return level color, used for visualization. (Show -> Advanced -> Level Coloration) */
	UE_API virtual FLinearColor GetLevelColor() const;

	/** Sets level color, used for visualization. (Show -> Advanced -> Level Coloration) */
	UE_API virtual void SetLevelColor(FLinearColor InColor);

	/** Whether level should be drawn in world composition view */
	UE_API virtual bool IsVisibleInCompositionView() const;
	
	/**	@return Whether level has associated blueprint script */
	UE_API bool HasKismet() const;

	/**	Opens level associated blueprint script */
	UE_API void OpenKismet();

	/** 
	 * Sets parent for this item
	 * @return false in case attaching has failed
	 */
	UE_API bool AttachTo(TSharedPtr<FLevelModel> InParent);

	/**	Notifies level model that filters has been changed */
	UE_API void OnFilterChanged();

	/**	@return Level child hierarchy */
	UE_API const FLevelModelList& GetChildren() const;
	
	/**	@return Parent level model */
	UE_API TSharedPtr<FLevelModel> GetParent() const;
	
	/**	Sets link to a parent model  */
	UE_API void SetParent(TSharedPtr<FLevelModel>);

	/**	Removes all entries from children list*/
	UE_API void RemoveAllChildren();

	/**	Removes specific child */
	UE_API void RemoveChild(TSharedPtr<FLevelModel> InChild);
	
	/**	Adds new entry to a children list */
	UE_API void AddChild(TSharedPtr<FLevelModel> InChild);
		
	/**	@return Whether this model has in ancestors specified level model */
	UE_API bool HasAncestor(const TSharedPtr<FLevelModel>& InLevel) const;

	/**	@return Whether this model has in descendants specified level model */
	UE_API bool HasDescendant(const TSharedPtr<FLevelModel>& InLevel) const;
	
	/** Returns the folder path that the level should use when displayed in the world hierarchy */
	virtual FName GetFolderPath() const { return NAME_None; }

	/** Sets the folder path that the level should use when displayed in the world hierarchy */
	virtual void SetFolderPath(const FName& InFolderPath) {}

	/** Returns true if the level model can be added to hierarchy folders */
	virtual bool HasFolderSupport() const { return false; }

	/**	@return Handles drop operation */
	UE_API virtual void OnDrop(const TSharedPtr<FLevelDragDropOp>& Op);
	
	/**	@return Whether it's possible to drop onto this level */
	UE_API virtual bool IsGoodToDrop(const TSharedPtr<FLevelDragDropOp>& Op) const;

	/** Notification when level was added(shown) to world */
	UE_API virtual void OnLevelAddedToWorld(ULevel* InLevel);

	/** Notification when level was removed(hidden) from world */
	UE_API virtual void OnLevelRemovedFromWorld();

	/** Notification on level reparenting */
	virtual void OnParentChanged() {};

	/** Event when level model has been changed */
	UE_API void BroadcastLevelChanged();
	
	/*
	 *
	 */
	struct FSimulationLevelStatus
	{
		bool bLoaded;
		bool bLoading;
		bool bVisible;
	};
	
	/** Updates this level simulation status  */
	UE_API void UpdateSimulationStatus(ULevelStreaming* StreamingLevel);

	/**	Broadcasts whenever level has changed */
	FSimpleEvent ChangedEvent;
	UE_API void BroadcastChangedEvent();

	/** Deselects all Actors in this level */
	UE_API void DeselectAllActors();

	/** Deselects all BSP surfaces in this level */
	UE_API void DeselectAllSurfaces();

	/**
	 *	Selects in the Editor all the Actors assigned to the Level, based on the specified conditions.
	 *
	 *	@param	bSelect					if true actors will be selected; If false, actors will be deselected
	 *	@param	bNotify					if true the editor will be notified of the selection change; If false, the editor will not
	 *	@param	bSelectEvenIfHidden		if true actors that are hidden will be selected; If false, they will be skipped
	 *	@param	Filter					Only actors which pass the filters restrictions will be selected
	 */
	UE_API void SelectActors(bool bSelect, bool bNotify, bool bSelectEvenIfHidden, 
						const TSharedPtr<ActorFilter>& Filter = TSharedPtr<ActorFilter>(NULL));
	
	/**
	 * Set the level to use external actors and convert its actors to external packaging or vice versa
	 * @param bUseExternal Convert to external if `bUseExternal` is true, convert to internal otherwise
	 */
	UE_API void ConvertLevelToExternalActors(bool bUseExternal);

	/**
	 * Return if the level can be converted to the specified external packaging mode.
	 * @param bToExternal The packaging mode we want to convert to. (i.e true: external, false: internal)
	 * @return true if we can convert the level to the `bToExternal` packaging mode.
	 */
	UE_API bool CanConvertLevelToExternalActors(bool bToExternal);

	/** Updates cached value of level actors count */
	UE_API void UpdateLevelActorsCount();

	/** Updates cached value of level display name */
	UE_API void UpdateDisplayName();

	/** @return the Level's Lightmass Size as a FString */
	UE_API FString GetLightmassSizeString() const;

	/** @return the Level's File Size as a FString */
	UE_API FString GetFileSizeString() const;
	
	/** @return Class used for streaming this level */
	UE_API virtual UClass* GetStreamingClass() const;

protected:
	/** Called when the map asset is renamed */
	UE_API void OnAssetRenamed(const FAssetData& AssetData, const FString& OldObjectPath);

protected:
	/** Level model display name */
	FString								DisplayName;

	/** Reference to owning collection model */
	FLevelCollectionModel&				LevelCollectionModel;
		
	/** The parent level  */
	TWeakPtr<FLevelModel>				Parent;
		
	/** Filtered children of this level  */
	FLevelModelList						FilteredChildren;

	/** All children of this level  */
	FLevelModelList						AllChildren;

	// Level simulation status
	FSimulationLevelStatus				SimulationStatus;

	// Whether this level model is selected
	bool								bSelected;
	
	// Whether this level model is expended in hierarchy view
	bool								bExpanded;

	// Whether this level model is in a process of loading content
	bool								bLoadingLevel;
	
	// Whether this level model does not pass filters
	bool								bFilteredOut;

	// Current translation delta
	FVector2D							LevelTranslationDelta;

	// Cached level actors count
	int32								LevelActorsCount;
};

#undef UE_API
