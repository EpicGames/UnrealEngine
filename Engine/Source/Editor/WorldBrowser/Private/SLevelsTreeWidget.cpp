// Copyright Epic Games, Inc. All Rights Reserved.

#include "SLevelsTreeWidget.h"

#include "Styling/AppStyle.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "LevelCollectionModel.h"
#include "AssetSelection.h"
#include "LevelModel.h"
#include "LevelCollectionModel.h"
#include "WorldBrowserDragDrop.h"
#include "WorldTreeItemTypes.h"
#include "SWorldHierarchyImpl.h"

#define LOCTEXT_NAMESPACE "WorldBrowser"

void SLevelsTreeWidget::Construct(const FArguments& InArgs, const TSharedPtr<FLevelCollectionModel>& InWorldModel, const TSharedPtr<SWorldHierarchyImpl>& InHierarchy)
{
	STreeView<TSharedPtr<WorldHierarchy::IWorldTreeItem>>::Construct(InArgs);

	WorldModel = InWorldModel;
	Hierarchy = InHierarchy;
}

FReply SLevelsTreeWidget::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<WorldHierarchy::FWorldBrowserDragDropOp> HierarchyOp = DragDropEvent.GetOperationAs<WorldHierarchy::FWorldBrowserDragDropOp>();

	if (HierarchyOp.IsValid())
	{
		// Assume that we're trying to attach the selection to the first persistent level
		FLevelModelList Models = WorldModel->GetRootLevelList();
		check(Models.Num() > 0);
		FText LevelName = FText::FromString(Models[0]->GetDisplayName());

		FText ToolTipText = FText::Format(LOCTEXT("OnDragHierarchyItemsOver_Invalid", "Cannot attach selected items to {0}"), LevelName);
		const FSlateBrush* ToolTipIcon = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.Error"));

		if (CanAttachAllItemsToRoot(HierarchyOp->GetDraggedItems()))
		{
			ToolTipText = FText::Format(LOCTEXT("OnDragHierarchyItemsOver_Success", "Attach selected items to {0}"), LevelName);
			ToolTipIcon = FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK"));
		}

		HierarchyOp->SetToolTip(ToolTipText, ToolTipIcon);

		return FReply::Handled();
	}
	else
	{
		TArray<FAssetData> AssetList;
		if (GetWorldAssetsFromDrag(DragDropEvent, AssetList) && DragDropEvent.GetOperation()->IsOfType<FAssetDragDropOp>())
		{
			TSharedPtr< FAssetDragDropOp > AssetOp = DragDropEvent.GetOperationAs< FAssetDragDropOp >();
			AssetOp->SetToolTip(LOCTEXT("OnDragWorldAssetsOverFolder", "Add Level(s)"), FAppStyle::GetBrush(TEXT("Graph.ConnectorFeedback.OK")));

			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

void SLevelsTreeWidget::OnDragLeave(const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<WorldHierarchy::FWorldBrowserDragDropOp> HierarchyOp = DragDropEvent.GetOperationAs<WorldHierarchy::FWorldBrowserDragDropOp>();
	TSharedPtr< FAssetDragDropOp > AssetOp = DragDropEvent.GetOperationAs< FAssetDragDropOp >();
	
	if (AssetOp.IsValid())
	{
		AssetOp->ResetToDefaultToolTip();
	}
	else if (HierarchyOp.IsValid())
	{
		HierarchyOp->ResetToDefaultToolTip();
	}
}

FReply SLevelsTreeWidget::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<WorldHierarchy::FWorldBrowserDragDropOp> HierarchyOp = DragDropEvent.GetOperationAs<WorldHierarchy::FWorldBrowserDragDropOp>();

	if (HierarchyOp.IsValid())
	{
		TSharedPtr<SWorldHierarchyImpl> HierarchyPtr = Hierarchy.Pin();

		if (HierarchyPtr.IsValid() && CanAttachAllItemsToRoot(HierarchyOp->GetDraggedItems()))
		{
			// Move any dropped items to the root
			HierarchyPtr->MoveDroppedItems(HierarchyOp->GetDraggedItems(), NAME_None);
			return FReply::Handled();
		}
	}
	else if (WorldModel.IsValid())
	{
		// Handle adding dropped levels to world
		TArray<FAssetData> AssetList;
		if (GetWorldAssetsFromDrag(DragDropEvent, AssetList))
		{
			WorldModel->AddExistingLevelsFromAssetData(AssetList);
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

bool SLevelsTreeWidget::GetWorldAssetsFromDrag(const FDragDropEvent& DragDropEvent, TArray<FAssetData>& OutWorldAssetList)
{
	TArray<FAssetData> AssetList = AssetUtil::ExtractAssetDataFromDrag(DragDropEvent);
	for (const auto& AssetData : AssetList)
	{
		if (AssetData.AssetClassPath == UWorld::StaticClass()->GetClassPathName())
		{
			OutWorldAssetList.Add(AssetData);
		}
	}

	return OutWorldAssetList.Num() > 0;
}

bool SLevelsTreeWidget::CanAttachAllItemsToRoot(const TArray<WorldHierarchy::FWorldTreeItemPtr>& Items) const
{
	bool bCanAttach = Items.Num() > 0;

	for (WorldHierarchy::FWorldTreeItemPtr Item : Items)
	{
		bCanAttach = Item->CanChangeParents();

		if (!bCanAttach)
		{
			break;
		}
	}

	return bCanAttach;
}

#undef LOCTEXT_NAMESPACE
