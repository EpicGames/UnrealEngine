// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

#include "Widgets/Views/STreeView.h"
#include "AssetRegistry/AssetData.h"

#include "IWorldTreeItem.h"

class FLevelModel;
class FLevelCollectionModel;
class SWorldHierarchyImpl;

class SLevelsTreeWidget : public STreeView<WorldHierarchy::FWorldTreeItemPtr>
{
public:
	void Construct(const FArguments& InArgs, const TSharedPtr<FLevelCollectionModel>& InWorldModel, const TSharedPtr<SWorldHierarchyImpl>& InHierachy);

	virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

	virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;

	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

private:
	/** Finds any world assets that are contained in the drag drop event and returns them in the supplied array. Returns true if a world asset was found */
	bool GetWorldAssetsFromDrag(const FDragDropEvent& DragDropEvent, TArray<FAssetData>& OutWorldAssetList);

	/** Checks to see if all selected items can be moved. */
	bool CanAttachAllItemsToRoot(const TArray<WorldHierarchy::FWorldTreeItemPtr>& Items) const;

private:
	TSharedPtr<FLevelCollectionModel> WorldModel;

	TWeakPtr<SWorldHierarchyImpl> Hierarchy;
};