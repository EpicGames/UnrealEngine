// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPropertyEditorLevelPackage.h"

#include "Containers/BitArray.h"
#include "Containers/Set.h"
#include "Containers/SparseArray.h"
#include "DetailLayoutBuilder.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Views/ITypedTableView.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformFileManager.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Math/Color.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "Misc/TextFilter.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Types/SlateStructs.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class ITableRow;
class STableViewBase;
class SWidget;

#define LOCTEXT_NAMESPACE "WorldBrowser"

void SPropertyEditorLevelPackage::Construct(const FArguments& InArgs, const TSharedPtr<IPropertyHandle>& InPropertyHandle)
{
	PropertyHandle			= InPropertyHandle;
	RootPath				= InArgs._RootPath;
	bSortAlphabetically		= InArgs._SortAlphabetically;
	OnShouldFilterPackage	= InArgs._OnShouldFilterPackage;

	// Setup packages text filter
	SearchBoxLevelPackageFilter = MakeShareable( 
		new LevelPackageTextFilter(LevelPackageTextFilter::FItemToStringArray::CreateSP(this, &SPropertyEditorLevelPackage::TransformPackageItemToString)) 
	);
	SearchBoxLevelPackageFilter->OnChanged().AddSP(this, &SPropertyEditorLevelPackage::OnTextFilterChanged);
	

	ChildSlot
	[
		SAssignNew(PropertyMainWidget, SComboButton)
		.ButtonStyle(FAppStyle::Get(), "PropertyEditor.AssetComboStyle")
		.ForegroundColor(FAppStyle::GetColor("PropertyEditor.AssetName.ColorAndOpacity"))
		.OnGetMenuContent( this, &SPropertyEditorLevelPackage::GetMenuContent )
		.ContentPadding(2.0f)
		.ButtonContent()
		[
			// Show the name of the asset or actor
			SNew(STextBlock)
			.TextStyle(FAppStyle::Get(), "PropertyEditor.AssetClass")
			.Font(IDetailLayoutBuilder::GetDetailFont())
			.Text(this, &SPropertyEditorLevelPackage::GetDisplayPackageName)
		]
	];
}

FText SPropertyEditorLevelPackage::GetDisplayPackageName() const
{
	FName PropertyValue;
	if (PropertyHandle->GetValue(PropertyValue) == FPropertyAccess::MultipleValues)
	{
		return LOCTEXT("MultipleValues", "Multiple Values");
	}
	else
	{
		if (PropertyValue != NAME_None)
		{
			FString LongPackageName = PropertyValue.ToString();
			if (LongPackageName.StartsWith(RootPath))
			{
				LongPackageName.RightChopInline(RootPath.Len()-1, EAllowShrinking::No); // do not chop front '/' from display name
				return FText::FromString(LongPackageName);
			}
		}
	}

	return FText::FromName(PropertyValue);
}

FString SPropertyEditorLevelPackage::GetPropertyValue() const
{
	FName PropertyValue; PropertyHandle->GetValue(PropertyValue);
	if (PropertyValue != NAME_None)
	{
		return PropertyValue.ToString();
	}

	return FString();
}

void SPropertyEditorLevelPackage::OnSelectionChanged(const TSharedPtr<FLevelPackageItem> Item, ESelectInfo::Type SelectInfo)
{
	if (Item.IsValid())
	{
		if (GetPropertyValue() == Item->LongPackageName)
		{
			return;	
		}

		PropertyHandle->SetValue(FName(*Item->LongPackageName));
		PropertyMainWidget->SetIsOpen(false);
	}
}

TSharedRef<SWidget> SPropertyEditorLevelPackage::GetMenuContent()
{
	PopulatePackages();
	return MakePickerWidget();
}

TSharedRef<SWidget> SPropertyEditorLevelPackage::MakePickerWidget()
{
	TSharedPtr<SLevelPackageListView>	PikerListView;
	TSharedRef<SWidget>					PikerWidget =

		SNew(SBox)
			.WidthOverride(300)
			.HeightOverride(300)
			[
				SNew(SVerticalBox)

				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew( SSearchBox )
					.ToolTipText(LOCTEXT("LevelPackage_FilterTooltip", "Type here to search levels"))
					.HintText(LOCTEXT( "LevelPackage_FilterHint", "Search Levels" ))
					.OnTextChanged(SearchBoxLevelPackageFilter.Get(), &LevelPackageTextFilter::SetRawFilterText)
				]
				+SVerticalBox::Slot()
				.FillHeight(1.f)
				[
					SAssignNew(PikerListView, SListView<TSharedPtr<FLevelPackageItem>>)
					.ListItemsSource(&FilteredLevelPackages)
					.SelectionMode(ESelectionMode::Single)
					.OnGenerateRow(this, &SPropertyEditorLevelPackage::MakeListRowWidget)
					.OnSelectionChanged(this, &SPropertyEditorLevelPackage::OnSelectionChanged)
				]
			];

	
	// Set current property value as selected in list widget
	TSharedPtr<FLevelPackageItem> CurrentItem = FindPackageItem(GetPropertyValue());
	if (CurrentItem.IsValid())
	{
		PikerListView->SetSelection(CurrentItem);
		PikerListView->RequestScrollIntoView(CurrentItem);
	}

	// store a weak pointer to a ListView to be able refresh it on filter changes
	PickerListWidget = PikerListView;
		
	return PikerWidget;
}

TSharedRef<ITableRow> SPropertyEditorLevelPackage::MakeListRowWidget(TSharedPtr<FLevelPackageItem> InPackageItem, 
																	 const TSharedRef<STableViewBase>& OwnerTable) const
{
	return SNew(STableRow<TSharedPtr<FString>>, OwnerTable)
		[
			SNew(SBorder)
			.Padding(5)
			.BorderImage(FAppStyle::GetBrush("NoBrush"))
			[
				SNew(STextBlock)
				.ToolTipText(FText::FromString(InPackageItem->LongPackageName))
				.Text(FText::FromString(InPackageItem->DisplayName))
				
			]
		];
}

void SPropertyEditorLevelPackage::OnTextFilterChanged()
{
	PopulateFilteredPackages();
	
	// Refresh picker list
	auto Picker = PickerListWidget.Pin();
	if (Picker.IsValid())
	{
		Picker->RequestListRefresh();
	}
}

FLevelPackageItem SPropertyEditorLevelPackage::PackageNameToItem(const FString& PackageName) const
{
	FLevelPackageItem Item;
	
	Item.LongPackageName = PackageName;
				
	// DisplayString for a package should be just pacakage name without path
	Item.DisplayName = FPackageName::GetShortName(Item.LongPackageName);
	
	return Item;
}

TSharedPtr<FLevelPackageItem> SPropertyEditorLevelPackage::FindPackageItem(const FString& PackageName)
{
	for (auto It = LevelPackages.CreateConstIterator(); It; ++It)
	{
		if ((*It)->LongPackageName == PackageName)
		{
			return (*It);
		}
	}
	
	return TSharedPtr<FLevelPackageItem>();
}

void SPropertyEditorLevelPackage::PopulatePackages()
{
	struct FWorldRootVisitor
		: public IPlatformFile::FDirectoryVisitor
	{
		const SPropertyEditorLevelPackage&		Owner;
		TArray<TSharedPtr<FLevelPackageItem>>&	Output;
		
		FWorldRootVisitor(const SPropertyEditorLevelPackage& InOwner, TArray<TSharedPtr<FLevelPackageItem>>& InOutput)
			: Owner(InOwner)
			, Output(InOutput)
		{}

		virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory) override
		{
			FString FullPath = FilenameOrDirectory;

			// for all packages
			if (!bIsDirectory && FPaths::GetExtension(FullPath, true) == FPackageName::GetMapPackageExtension())
			{
				FLevelPackageItem Item = Owner.PackageNameToItem(FPackageName::FilenameToLongPackageName(FilenameOrDirectory));
				Output.Add(MakeShareable(new FLevelPackageItem(Item)));
			}

			return true;
		}
	};
	
	LevelPackages.Empty();
	FWorldRootVisitor Visitor(*this, LevelPackages);
	FPlatformFileManager::Get().GetPlatformFile().IterateDirectoryRecursively(*FPackageName::LongPackageNameToFilename(RootPath), Visitor);
	
	// populate items array according to current filter settings
	PopulateFilteredPackages();
}

void SPropertyEditorLevelPackage::PopulateFilteredPackages()
{
	FilteredLevelPackages.Empty(LevelPackages.Num());

	for (auto It = LevelPackages.CreateConstIterator(); It; ++It)
	{		
		if (!OnShouldFilterPackage.IsBound() || !OnShouldFilterPackage.Execute((*It)->LongPackageName))
		{
			if (SearchBoxLevelPackageFilter->PassesFilter((*It)))
			{
				FilteredLevelPackages.Add((*It));
			}
		}
	}
	
	// Sort filtered packages if client wants to
	if (bSortAlphabetically)
	{
		struct FSortPredicate
		{
			bool operator()(const TSharedPtr<FLevelPackageItem>& A, const TSharedPtr<FLevelPackageItem>& B) const
			{
				return A->DisplayName < B->DisplayName;
			}
		};
		
		FilteredLevelPackages.Sort(FSortPredicate());
	}
}

void SPropertyEditorLevelPackage::TransformPackageItemToString(const TSharedPtr<FLevelPackageItem>& Item, TArray<FString>& OutSearchStrings) const
{
	if (Item.IsValid())
	{
		OutSearchStrings.Add(Item->DisplayName);
	}
}

#undef LOCTEXT_NAMESPACE
