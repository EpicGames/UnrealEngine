// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "Internationalization/Text.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Templates/SharedPointer.h"
#include "Templates/TypeHash.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class ITableRow;
class STableViewBase;
class SWidget;
template <typename ItemType> class SListView;
template< typename ItemType > class TTextFilter;

/////////////////////////////////////////////////////
// FLevelPackageItem
struct FLevelPackageItem
{
	FString DisplayName;
	FString LongPackageName;
};

typedef TTextFilter<const TSharedPtr<FLevelPackageItem>&>	LevelPackageTextFilter;
typedef SListView<TSharedPtr<FLevelPackageItem>>			SLevelPackageListView;	

/////////////////////////////////////////////////////
// SPropertyEditorLevelPackage
// 
// Widget which plugs in to a details panel to edit FNames properties which represent a level package name
// Looks kinda similar to a SPropertyEditorAsset, should be replaced with it when ContentBrowser will treat ULevel as an asset
//
class SPropertyEditorLevelPackage
	: public SCompoundWidget
{
public:	
	DECLARE_DELEGATE_RetVal_OneParam(bool, FOnShouldFilterPackage, const FString&);
	DECLARE_DELEGATE_OneParam(FOnPackagePicked, const FString&);


	SLATE_BEGIN_ARGS( SPropertyEditorLevelPackage )
		: _RootPath(FPackageName::FilenameToLongPackageName(FPaths::ProjectContentDir())) 
		, _SortAlphabetically(false)
		{}
		/** Root folder path for gathering level packages */	
		SLATE_ARGUMENT( FString, RootPath )
		/** Whether package list should be aaranged alphabetically */
		SLATE_ARGUMENT( bool, SortAlphabetically )
		/** Called to check if an item should be filtered out by external code */
		SLATE_EVENT(FOnShouldFilterPackage, OnShouldFilterPackage)
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs, const TSharedPtr<class IPropertyHandle>& InPropertyHandle);
	
	/**@return FLevelPackageItem initialized from a given PackageName */
	FLevelPackageItem PackageNameToItem(const FString& PackageName) const;

private:
	/**@return Display text for a property value */
	FText GetDisplayPackageName() const;
	
	/**@return Picker widget with content to display combo box drop menu */
	TSharedRef<SWidget> GetMenuContent();
	
	/** Creates picker widget */
	TSharedRef<SWidget> MakePickerWidget();
	
	/** Creates a row for a piker widget */
	TSharedRef<ITableRow> MakeListRowWidget(TSharedPtr<FLevelPackageItem> InPackageName, const TSharedRef<STableViewBase>& OwnerTable) const;
	
	/** Handles on OnSelectionChanged event from a picker widget */
	void OnSelectionChanged(const TSharedPtr<FLevelPackageItem> Item, ESelectInfo::Type SelectInfo);
	
	/** Populates internal array with levels packages found on disk under RootPath */
	void PopulatePackages();
	
	/** Populates internal array with packages previously found on disk according to current filter settings  */
	void PopulateFilteredPackages();
	
	/** Transforms FLevelPackageItem to a search terms for a text filter */
	void TransformPackageItemToString(const TSharedPtr<FLevelPackageItem>& Item, TArray<FString>& OutSearchStrings) const;
	
	/** Handles text filter changes */
	void OnTextFilterChanged();
	
	/** Handles text filter changes */
	TSharedPtr<FLevelPackageItem> FindPackageItem(const FString& PackageName);
	
	/** @return Current property value as a string */
	FString GetPropertyValue() const;

private:
	/**	 */
	FString									RootPath;
	/**	 */
	bool									bSortAlphabetically;
	/**	 */
	FOnShouldFilterPackage					OnShouldFilterPackage;
	/**	 */
	TSharedPtr<class IPropertyHandle>		PropertyHandle;
	/**	 */
	TSharedPtr<class SComboButton>			PropertyMainWidget;
	/**	 */
	TWeakPtr<SLevelPackageListView>			PickerListWidget;
	/**	 */
	TArray<TSharedPtr<FLevelPackageItem>>	LevelPackages;
	/**	 */
	TArray<TSharedPtr<FLevelPackageItem>>	FilteredLevelPackages;
	/**	 */
	TSharedPtr<LevelPackageTextFilter>		SearchBoxLevelPackageFilter;

};

