// Copyright Epic Games, Inc. All Rights Reserved.
#include "SWorldDetails.h"
#include "Styling/AppStyle.h"
#include "LevelCollectionModel.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "WorldBrowserModule.h"
#include "SSimpleButton.h"

#include "IDetailsView.h"
#include "LevelEditor.h"
#include "PropertyEditorModule.h"

#define LOCTEXT_NAMESPACE "WorldBrowser"

SWorldDetails::SWorldDetails()
	: bUpdatingSelection(false)
{
}

SWorldDetails::~SWorldDetails()
{
	GLevelEditorModeTools().DeactivateMode(FBuiltinEditorModes::EM_StreamingLevel);
	
	FWorldBrowserModule& WorldBrowserModule = FModuleManager::GetModuleChecked<FWorldBrowserModule>("WorldBrowser");
	WorldBrowserModule.OnBrowseWorld.RemoveAll(this);
	
	OnBrowseWorld(nullptr);
}

void SWorldDetails::Construct(const FArguments& InArgs)
{
	FWorldBrowserModule& WorldBrowserModule = FModuleManager::GetModuleChecked<FWorldBrowserModule>("WorldBrowser");
	WorldBrowserModule.OnBrowseWorld.AddSP(this, &SWorldDetails::OnBrowseWorld);
	
	OnBrowseWorld(InArgs._InWorld);
}

void SWorldDetails::OnBrowseWorld(UWorld* InWorld)
{
	// Remove old world bindings
	ChildSlot
	[
		SNullWidget::NullWidget
	];
	
	if (WorldModel.IsValid())
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		WorldModel->UnregisterDetailsCustomization(PropertyModule, DetailsView);
		WorldModel->SelectionChanged.RemoveAll(this);
		WorldModel->CollectionChanged.RemoveAll(this);
	}
	
	WorldModel = nullptr;

	// Bind to a new world
	if (InWorld)
	{
		FWorldBrowserModule& WorldBrowserModule = FModuleManager::GetModuleChecked<FWorldBrowserModule>("WorldBrowser");
		WorldModel = WorldBrowserModule.SharedWorldModel(InWorld);

		WorldModel->SelectionChanged.AddSP(this, &SWorldDetails::OnSelectionChanged);
		WorldModel->CollectionChanged.AddSP(this, &SWorldDetails::OnCollectionChanged);
	
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		FDetailsViewArgs Args;
		Args.bAllowSearch = false;
		Args.NameAreaSettings = FDetailsViewArgs::HideNameArea;
		Args.bHideSelectionTip = true;
	
		DetailsView = PropertyModule.CreateDetailView(Args);
		ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
			.Padding(FMargin(8.f))
			[
				SAssignNew(VerticalBox, SVerticalBox)

				// Inspect level box
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)

					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Left)
					[
						SNew(STextBlock)
						.Text(FText::FromString(TEXT("Inspect level:")))
					]
					
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Left)
					.Padding(4.f, 0.f, 0.f, 0.f)
					[
						SAssignNew(SubLevelsComboBox, SComboBox<TSharedPtr<FLevelModel>>)
						.OptionsSource(&WorldModel->GetFilteredLevels())
						.OnSelectionChanged(this, &SWorldDetails::OnSetInspectedLevel)
						.OnGenerateWidget(this, &SWorldDetails::HandleInspectedLevelComboBoxGenerateWidget)
						.Content()
						[
							SNew(STextBlock)
							.Text(this, &SWorldDetails::GetInspectedLevelText)
						]
					]
					
					// Button to summon levels hierarchy tab
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Left)
					.Padding(4.f, 0.f, 0.f, 0.f)
					[
						SNew(SSimpleButton)
						.OnClicked(this, &SWorldDetails::OnSummonHierarchy)
						.ToolTipText(LOCTEXT("SummonHierarchyToolTipText", "Summons levels hierarchy"))
						.Icon(FAppStyle::Get().GetBrush("WorldBrowser.HierarchyButtonBrush"))
					]

					// Button to summon world composition tab
					+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Left)
					[
						SNew(SSimpleButton)
						.Visibility(this, &SWorldDetails::GetCompositionButtonVisibility)
						.OnClicked(this, &SWorldDetails::OnSummonComposition)
						.ToolTipText(LOCTEXT("SummonCompositionToolTipText", "Summons world composition"))
						.Icon(FAppStyle::Get().GetBrush("WorldBrowser.CompositionButtonBrush"))
					]
				]
			
				// Level details
				+SVerticalBox::Slot()
				.FillHeight(1.f)
				.Padding(0,4,0,0)
				[
					DetailsView.ToSharedRef()
				]
			]
		];

		WorldModel->RegisterDetailsCustomization(PropertyModule, DetailsView);

		// Current selection
		OnSelectionChanged();
	}
}

void SWorldDetails::OnSelectionChanged()
{
	bUpdatingSelection = true;
	
	auto SelectedLevels = WorldModel->GetSelectedLevels();
	TArray<UObject*> TileProperties;
	
	for (auto It = SelectedLevels.CreateIterator(); It; ++It)
	{
		UObject* PropertiesObject = (*It)->GetNodeObject();
		if (PropertiesObject)
		{
			TileProperties.Add(PropertiesObject);
		}
	}

	DetailsView->SetObjects(TileProperties, true);

	if (VerticalBoxBorder.IsValid())
	{
		VerticalBox->RemoveSlot(VerticalBoxBorder->AsShared());
		VerticalBoxBorder.Reset();
	}

	if (SelectedLevels.Num() == 0 || SelectedLevels.Num() > 1)
	{
		// Clear ComboBox selection in case we have multiple selection
		SubLevelsComboBox->ClearSelection();
	}
	else
	{
		SubLevelsComboBox->SetSelectedItem(SelectedLevels[0]);
	}

	bUpdatingSelection = false;
}

void SWorldDetails::OnCollectionChanged()
{
//	SubLevelsComboBox->RefreshOptions();
}

void SWorldDetails::OnSetInspectedLevel(TSharedPtr<FLevelModel> InLevelModel, ESelectInfo::Type SelectInfo)
{
	if (!bUpdatingSelection && InLevelModel.IsValid())
	{
		FLevelModelList SelectedLevels; SelectedLevels.Add(InLevelModel);
		WorldModel->SetSelectedLevels(SelectedLevels);
	}
}

TSharedRef<SWidget> SWorldDetails::HandleInspectedLevelComboBoxGenerateWidget(TSharedPtr<FLevelModel> InLevelModel) const
{
	return SNew(SBox)
	.Padding(4)
	[
		SNew(STextBlock).Text(FText::FromString(InLevelModel->GetDisplayName()))
	];
}

FText SWorldDetails::GetInspectedLevelText() const
{
	const FLevelModelList& SelectedLevels = WorldModel->GetSelectedLevels();
	
	if (SelectedLevels.Num() > 1)
	{
		return LOCTEXT("MultipleInspectedLevelText", "Multiple Values");
	}
	else if (SelectedLevels.Num() == 1)
	{
		return FText::FromString(SelectedLevels[0]->GetDisplayName());
	}

	return LOCTEXT("EmptyInspectedLevelText", "None");
}

FReply SWorldDetails::OnSummonHierarchy()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>( "LevelEditor" );
	LevelEditorModule.SummonWorldBrowserHierarchy();
	return FReply::Handled();
}

EVisibility SWorldDetails::GetCompositionButtonVisibility() const
{
	return WorldModel->IsTileWorld() ? EVisibility::Visible : EVisibility::Collapsed;
}

FReply SWorldDetails::OnSummonComposition()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>( "LevelEditor" );
	LevelEditorModule.SummonWorldBrowserComposition();
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
