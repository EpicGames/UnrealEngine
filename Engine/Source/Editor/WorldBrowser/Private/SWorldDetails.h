// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SComboBox.h"

class FLevelCollectionModel;
class FLevelModel;
class IDetailsView;

//----------------------------------------------------------------
//
//
//----------------------------------------------------------------
class SWorldDetails
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SWorldDetails)
		:_InWorld(nullptr)
		{}
		SLATE_ARGUMENT(UWorld*, InWorld)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	SWorldDetails();
	~SWorldDetails();

private:
	/**  */
	void OnBrowseWorld(UWorld* InWorld);

	/** Handles selection changes in data source */
	void OnSelectionChanged();

	/** Handles levels collection changes in data source */
	void OnCollectionChanged();

	/**  */
	void OnSetInspectedLevel(TSharedPtr<FLevelModel> InLevelModel, ESelectInfo::Type SelectInfo);
	TSharedRef<SWidget> HandleInspectedLevelComboBoxGenerateWidget(TSharedPtr<FLevelModel> InLevelModel) const;
	FText GetInspectedLevelText() const;
	
	/**  */
	FReply OnSummonHierarchy();

	/**  */
	EVisibility GetCompositionButtonVisibility() const;
	FReply OnSummonComposition();
		
private:
	TSharedPtr<FLevelCollectionModel>				WorldModel;
	TSharedPtr<IDetailsView>						DetailsView;
	TSharedPtr<SVerticalBox>						VerticalBox;
	TSharedPtr<SBorder>								VerticalBoxBorder;
	TSharedPtr<SComboBox<TSharedPtr<FLevelModel>>>	SubLevelsComboBox;
	bool											bUpdatingSelection;	
};
