// Copyright Epic Games, Inc. All Rights Reserved.

#include "SWorldHierarchy.h"
#include "Modules/ModuleManager.h"
#include "Misc/PackageName.h"
#include "Widgets/SBoxPanel.h"
#include "Textures/SlateIcon.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "WorldPartition/WorldPartitionSubsystem.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Styling/AppStyle.h"
#include "WorldBrowserModule.h"
#include "LevelModel.h"
#include "LevelCollectionModel.h"
#include "SSimpleButton.h"
#include "SSimpleComboButton.h"

#include "LevelEditor.h"

#include "DragAndDrop/AssetDragDropOp.h"

#include "SWorldHierarchyImpl.h"
#include "WorldBrowserConfig.h"

#define LOCTEXT_NAMESPACE "WorldBrowser"

SWorldHierarchy::SWorldHierarchy()
{
}

SWorldHierarchy::~SWorldHierarchy()
{
	FWorldBrowserModule& WorldBrowserModule = FModuleManager::GetModuleChecked<FWorldBrowserModule>("WorldBrowser");
	WorldBrowserModule.OnBrowseWorld.RemoveAll(this);
}

void SWorldHierarchy::Construct(const FArguments& InArgs)
{
	FWorldBrowserModule& WorldBrowserModule = FModuleManager::GetModuleChecked<FWorldBrowserModule>("WorldBrowser");
	WorldBrowserModule.OnBrowseWorld.AddSP(this, &SWorldHierarchy::OnBrowseWorld);

	OnBrowseWorld(InArgs._InWorld);
}

bool SWorldHierarchy::IsColumnVisible(FName Column) const
{
	const TSharedPtr<SWorldHierarchyImpl> WidgetPin = WeakWorldHierarchyImpl.Pin();
	return WidgetPin
		? WidgetPin->IsColumnVisible(Column)
		: SWorldHierarchyImpl::IsVisibleInConfig(Column);
}

void SWorldHierarchy::SetColumnVisible(FName Column, bool bVisible)
{
	if (const TSharedPtr<SWorldHierarchyImpl> WidgetPin = WeakWorldHierarchyImpl.Pin())
	{
		WidgetPin->SetColumnVisible(Column, bVisible);
	}
	else
	{
		// By documentation, SetColumnVisible does not save into the config. Only direct user action should do that.
		SWorldHierarchyImpl::SetWillBeVisibleInConfigTransient(Column, bVisible);
	}
}

void SWorldHierarchy::OnBrowseWorld(UWorld* InWorld)
{
	// Remove all binding to an old world
	ChildSlot
	[
		SNullWidget::NullWidget
	];
	WorldModel = nullptr;
		
	// Bind to a new world
	if (InWorld)
	{
		if (UWorld::IsPartitionedWorld(InWorld))
		{
			ChildSlot
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("FeatureDisabledWithWorldPartition", "This feature is disabled when World Partition is enabled."))
				]
			];
		}
		else
		{

			FWorldBrowserModule& WorldBrowserModule = FModuleManager::GetModuleChecked<FWorldBrowserModule>("WorldBrowser");
			WorldModel = WorldBrowserModule.SharedWorldModel(InWorld);

			ChildSlot
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
				[

					SNew(SVerticalBox)

					// Toolbar
					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SHorizontalBox)
					
						// Toolbar
						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Left)
						.Padding(8.f, 0.f, 0.f, 0.f)
						[
							// Levels menu
							SNew( SSimpleComboButton )
							.OnGetMenuContent(this, &SWorldHierarchy::GetFileButtonContent)
							.Icon(FAppStyle::Get().GetBrush("WorldBrowser.LevelsMenuBrush"))
							.Text(LOCTEXT("LevelsButton", "Levels"))
							.HasDownArrow(true)
						]

						// Button to summon level details tab
						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Left)
						.Padding(8.f, 0.f, 0.f, 0.f)
						[
							SNew(SSimpleButton)
							.OnClicked(this, &SWorldHierarchy::OnSummonDetails)
							.ToolTipText(LOCTEXT("SummonDetailsToolTipText", "Summons level details"))
							.Icon(FAppStyle::Get().GetBrush("WorldBrowser.DetailsButtonBrush"))
						]

						// Button to summon world composition tab
						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Left)
						.Padding(8.f, 0.f, 0.f, 0.f)
						[
							SNew(SSimpleButton)
							.Visibility(this, &SWorldHierarchy::GetCompositionButtonVisibility)
							.OnClicked(this, &SWorldHierarchy::OnSummonComposition)
							.ToolTipText(LOCTEXT("SummonCompositionToolTipText", "Summons world composition"))
							.Icon(this, &SWorldHierarchy::GetSummonCompositionBrush)
						]
					]
			
					// Hierarchy
					+SVerticalBox::Slot()
					.FillHeight(1.f)
					.Padding(0.f, 4.f, 0.f ,0.f)
					[
						SAssignNew(WeakWorldHierarchyImpl, SWorldHierarchyImpl)
						.InWorldModel(WorldModel)
					]
				]
			];
		}
	}
}

FReply SWorldHierarchy::OnSummonDetails()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>( "LevelEditor" );
	LevelEditorModule.SummonWorldBrowserDetails();
	return FReply::Handled();
}

EVisibility SWorldHierarchy::GetCompositionButtonVisibility() const
{
	return WorldModel->IsTileWorld() ? EVisibility::Visible : EVisibility::Collapsed;
}

FReply SWorldHierarchy::OnSummonComposition()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>( "LevelEditor" );
	LevelEditorModule.SummonWorldBrowserComposition();
	return FReply::Handled();
}

const FSlateBrush* SWorldHierarchy::GetSummonCompositionBrush() const
{
	return FAppStyle::GetBrush("WorldBrowser.CompositionButtonBrush");
}

TSharedRef<SWidget> SWorldHierarchy::GetFileButtonContent()
{
	FMenuBuilder MenuBuilder(true, WorldModel->GetCommandList());

	// let current level collection model fill additional 'File' commands
	WorldModel->CustomizeFileMainMenu(MenuBuilder);

	return MenuBuilder.MakeWidget();
}

#undef LOCTEXT_NAMESPACE
