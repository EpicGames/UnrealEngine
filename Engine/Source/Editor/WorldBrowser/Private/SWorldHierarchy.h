// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "IWorldHierarchy.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FLevelCollectionModel;
class SWorldHierarchyImpl;
struct FSlateBrush;

/** Listens for world changes and updates SWorldHierarchyImpl, which displays the level hierarchy for the passed in world. */
class SWorldHierarchy
	: public SCompoundWidget
	, public UE::WorldHierarchy::IWorldHierarchy
{
	SLATE_BEGIN_ARGS(SWorldHierarchy)
		:_InWorld(nullptr)
		{}
		SLATE_ARGUMENT(UWorld*, InWorld)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	SWorldHierarchy();
	~SWorldHierarchy();

	//~ Begin IWorldHierarchy Interface
	virtual TSharedRef<SWidget> GetWidget() override { return SharedThis(this); }
	virtual bool IsColumnVisible(FName Column) const override;
	virtual void SetColumnVisible(FName Column, bool bVisible) override;
	//~ End IWorldHierarchy Interface

private:
	void OnBrowseWorld(UWorld* InWorld);

	/**  */
	FReply OnSummonDetails();
	/**  */
	EVisibility GetCompositionButtonVisibility() const;
	FReply OnSummonComposition();
	const FSlateBrush* GetSummonCompositionBrush() const;

	/**  */
	TSharedRef<SWidget> GetFileButtonContent();

private:
	
	/**
	 * Model for the UI managing the world logic.
	 * FWorldBrowserModule expects to be the only referencer of this model when switching worlds.
	 */
	TSharedPtr<FLevelCollectionModel> WorldModel;
	/**
	 * Actually displays the hierarchy.
	 *
	 * WorldModel, which is managed by FWorldBrowserModule, expects the reference count to be 1 when switching worlds.
	 * OnBrowseWorld kills this widget when switching worlds.
	 * That is why this must be a weak pointer instead of a strong pointer.
	 */
	TWeakPtr<SWorldHierarchyImpl> WeakWorldHierarchyImpl;
};
