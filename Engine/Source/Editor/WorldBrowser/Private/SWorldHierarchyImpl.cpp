// Copyright Epic Games, Inc. All Rights Reserved.

#include "SWorldHierarchyImpl.h"

#include "SLevelsTreeWidget.h"
#include "SWorldHierarchyItem.h"
#include "SWorldHierarchy.h"

#include "WorldBrowserModule.h"
#include "Modules/ModuleManager.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ToolMenus.h"
#include "UObject/ObjectSaveContext.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SButton.h"
#include "SSimpleComboButton.h"
#include "SSimpleButton.h"

#include "WorldTreeItemTypes.h"
#include "LevelFolders.h"
#include "ScopedTransaction.h"
#include "Templates/UnrealTemplate.h"
#include "Editor.h"
#include "WorldBrowserConfig.h"
#include "WorldBrowserStyle.h"
#include "WorldHierarchyColumns.h"
#include "RevisionControlStyle/RevisionControlStyle.h"

#define LOCTEXT_NAMESPACE "WorldBrowser"

DEFINE_LOG_CATEGORY_STATIC(LogWorldHierarchy, Log, All);

SWorldHierarchyImpl::SWorldHierarchyImpl()
	: bUpdatingSelection(false)
	, bIsReentrant(false)
	, bFullRefresh(true)
	, bNeedsRefresh(true)
	, bRebuildFolders(false)
	, bSortDirty(false)
	, bFoldersOnlyMode(false)
{
}

SWorldHierarchyImpl::~SWorldHierarchyImpl()
{
	GEditor->UnregisterForUndo(this);

	WorldModel->SelectionChanged.RemoveAll(this);
	WorldModel->HierarchyChanged.RemoveAll(this);
	WorldModel->CollectionChanged.RemoveAll(this);
	WorldModel->PreLevelsUnloaded.RemoveAll(this);

	if (SearchBoxLevelFilter.IsValid())
	{
		WorldModel->RemoveFilter(SearchBoxLevelFilter.ToSharedRef());
	}

	if (FLevelFolders::IsAvailable())
	{
		FLevelFolders& LevelFolders = FLevelFolders::Get();
		LevelFolders.OnFolderCreate.RemoveAll(this);
		LevelFolders.OnFolderMove.RemoveAll(this);
		LevelFolders.OnFolderDelete.RemoveAll(this);
	}

	FEditorDelegates::PostSaveWorldWithContext.RemoveAll(this);
}

void SWorldHierarchyImpl::Construct(const FArguments& InArgs)
{
	WorldModel = InArgs._InWorldModel;
	check(WorldModel.IsValid());

	WorldModel->SelectionChanged.AddSP(this, &SWorldHierarchyImpl::OnUpdateSelection);
	WorldModel->HierarchyChanged.AddSP(this, &SWorldHierarchyImpl::RebuildFoldersAndFullRefresh);
	WorldModel->CollectionChanged.AddSP(this, &SWorldHierarchyImpl::RebuildFoldersAndFullRefresh);
	WorldModel->PreLevelsUnloaded.AddSP(this, &SWorldHierarchyImpl::OnBroadcastLevelsUnloaded);
	
	bFoldersOnlyMode = InArgs._ShowFoldersOnly;
	ExcludedFolders = InArgs._InExcludedFolders;
	OnItemPicked = InArgs._OnItemPickedDelegate;

	if (!bFoldersOnlyMode)
	{
		SearchBoxLevelFilter = MakeShareable(new LevelTextFilter(
			LevelTextFilter::FItemToStringArray::CreateSP(this, &SWorldHierarchyImpl::TransformLevelToString)
		));
	}

	SearchBoxHierarchyFilter = MakeShareable(new HierarchyFilter(
		HierarchyFilter::FItemToStringArray::CreateSP(this, &SWorldHierarchyImpl::TransformItemToString)
	));

	// Might be overkill to have both filters call full refresh on change, but this should just request a full refresh
	//	twice instead of actually performing the refresh itself.
	if (SearchBoxLevelFilter.IsValid())
	{
		SearchBoxLevelFilter->OnChanged().AddSP(this, &SWorldHierarchyImpl::FullRefresh);
	}
	SearchBoxHierarchyFilter->OnChanged().AddSP(this, &SWorldHierarchyImpl::FullRefresh);
	
	FOnContextMenuOpening ContextMenuEvent;
	if (!bFoldersOnlyMode)
	{
		ContextMenuEvent = FOnContextMenuOpening::CreateSP(this, &SWorldHierarchyImpl::ConstructLevelContextMenu);
	}

	TSharedRef<SWidget> CreateNewFolderButton = SNullWidget::NullWidget;
	if (!bFoldersOnlyMode)
	{
		CreateNewFolderButton = 
			SNew(SSimpleButton)
			.ToolTipText(LOCTEXT("CreateFolderTooltip", "Create a new folder containing the current selection"))
			.OnClicked(this, &SWorldHierarchyImpl::OnCreateFolderClicked)
			.Visibility(WorldModel->HasFolderSupport() ? EVisibility::Visible : EVisibility::Collapsed)
			.Icon(FAppStyle::Get().GetBrush("WorldBrowser.NewFolderIcon"));
	}

	ChildSlot
	.Padding(8.f, 0.f, 8.f, 0.f)
	[
		SNew(SVerticalBox)
		// Hierarchy Toolbar
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			
			// Filter box
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			[
				SNew(SSearchBox)
				.ToolTipText(LOCTEXT("FilterSearchToolTip", "Type here to search Levels"))
				.HintText(LOCTEXT("FilterSearchHint", "Search Levels"))
				.OnTextChanged(this, &SWorldHierarchyImpl::SetFilterText)
			]

			// Create New Folder icon
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(2.0f, 0.0f, 0.0f, 0.0f)
			[
				CreateNewFolderButton
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(2.0f, 0.0f, 0.0f, 0.0f)
			[
				SNew(SSimpleComboButton)
				.OnGetMenuContent( this, &SWorldHierarchyImpl::GetViewButtonContent )
				.Icon(FAppStyle::Get().GetBrush("Icons.Settings"))

			]
		]
		// Empty Label
		+SVerticalBox::Slot()
		.HAlign(HAlign_Center)
		[
			SNew(STextBlock)
			.Visibility(this, &SWorldHierarchyImpl::GetEmptyLabelVisibility)
			.Text(LOCTEXT("EmptyLabel", "Empty"))
			.ColorAndOpacity(FLinearColor(0.4f, 1.0f, 0.4f))
		]

		// Hierarchy
		+SVerticalBox::Slot()
		.FillHeight(1.f)
		.Padding(0.f, 4.f, 0.f, 0.f)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
			.Padding(FMargin(0.f, 4.f, 0.f, 0.f))
			[
				SAssignNew(TreeWidget, SLevelsTreeWidget, WorldModel, SharedThis(this))
				.TreeItemsSource(&RootTreeItems)
				.SelectionMode(ESelectionMode::Multi)
				.OnGenerateRow(this, &SWorldHierarchyImpl::GenerateTreeRow)
				.OnGetChildren(this, &SWorldHierarchyImpl::GetChildrenForTree)
				.OnSelectionChanged(this, &SWorldHierarchyImpl::OnSelectionChanged)
				.OnExpansionChanged(this, &SWorldHierarchyImpl::OnExpansionChanged)
				.OnMouseButtonDoubleClick(this, &SWorldHierarchyImpl::OnTreeViewMouseButtonDoubleClick)
				.OnContextMenuOpening(ContextMenuEvent)
				.OnItemScrolledIntoView(this, &SWorldHierarchyImpl::OnTreeItemScrolledIntoView)
				.HeaderRow(CreateHeaderRow())
			]
		]

		// Separator
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0, 0, 0, 1)
		[
			SNew(SSeparator)
			.Visibility(bFoldersOnlyMode ? EVisibility::Collapsed : EVisibility::Visible)
		]
		
		// View options
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			.Visibility(bFoldersOnlyMode ? EVisibility::Collapsed : EVisibility::Visible)

			// Asset count
			+SHorizontalBox::Slot()
			.FillWidth(1.f)
			.VAlign(VAlign_Center)
			.Padding(8.f)
			[
				SNew( STextBlock )
				.Text( this, &SWorldHierarchyImpl::GetFilterStatusText )
				.ColorAndOpacity( this, &SWorldHierarchyImpl::GetFilterStatusTextColor )
			]
		]
	];
	
	if (FLevelFolders::IsAvailable())
	{
		FLevelFolders& LevelFolders = FLevelFolders::Get();
		LevelFolders.OnFolderCreate.AddSP(this, &SWorldHierarchyImpl::OnBroadcastFolderCreate);
		LevelFolders.OnFolderMove.AddSP(this, &SWorldHierarchyImpl::OnBroadcastFolderMove);
		LevelFolders.OnFolderDelete.AddSP(this, &SWorldHierarchyImpl::OnBroadcastFolderDelete);

		if (!bFoldersOnlyMode)
		{
			FEditorDelegates::PostSaveWorldWithContext.AddSP(this, &SWorldHierarchyImpl::OnWorldSaved);
		}
	}

	if (SearchBoxLevelFilter.IsValid())
	{
		WorldModel->AddFilter(SearchBoxLevelFilter.ToSharedRef());
	}

	OnUpdateSelection();

	GEditor->RegisterForUndo(this);
}

void SWorldHierarchyImpl::Tick( const FGeometry& AllotedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	SCompoundWidget::Tick(AllotedGeometry, InCurrentTime, InDeltaTime);

	if (bNeedsRefresh)
	{
		if (!bIsReentrant)
		{
			Populate();
		}
	}

	if (bSortDirty)
	{
		SortItems(RootTreeItems);
		for (const auto& Pair : TreeItemMap)
		{
			Pair.Value->Flags.bChildrenRequiresSort = true;
		}

		bSortDirty = false;
	}
}

void SWorldHierarchyImpl::OnWorldSaved(UWorld* World, FObjectPostSaveContext ObjectSaveContext)
{
	if (FLevelFolders::IsAvailable())
	{
		for (TSharedPtr<FLevelModel> RootLevel : WorldModel->GetRootLevelList())
		{
			FLevelFolders::Get().SaveLevel(RootLevel.ToSharedRef());
		}
	}
}

void SWorldHierarchyImpl::RefreshView()
{
	bNeedsRefresh = true;
}

TSharedRef<SHeaderRow> SWorldHierarchyImpl::CreateHeaderRow()
{
	using namespace UE::WorldHierarchy;
	constexpr float IconWidth = 24.f;

	TSharedRef<SHeaderRow> HeaderRow = SAssignNew(HeaderRowWidget, SHeaderRow)
		.CanSelectGeneratedColumn(true) // Lets the user choose which columns should be shown
		.OnHiddenColumnsListChanged(this, &SWorldHierarchyImpl::SaveColumnVisibilitiesIntoConfig)

		/** Level visibility column */
		+ SHeaderRow::Column(HierarchyColumns::ColumnID_EditorVisibility)
		.Visibility(bFoldersOnlyMode ? EVisibility::Collapsed : EVisibility::Visible)
		.DefaultLabel(LOCTEXT("Column.Visibility.DefaultLabel", "Visibility"))
		.FixedWidth(IconWidth)
		.HeaderContent()
		[
			SNew(SImage)
			.Image(FAppStyle::Get().GetBrush("Level.VisibleIcon16x"))
			.ColorAndOpacity(FSlateColor::UseForeground())
			.ToolTipText(LOCTEXT("Column.Visibility.Tooltip", "Toggles the editor visibility"))
		]

		/** Level game visibility column */
		+ SHeaderRow::Column(HierarchyColumns::ColumnID_GameVisibility)
		.Visibility(bFoldersOnlyMode ? EVisibility::Collapsed : EVisibility::Visible)
		.DefaultLabel(LOCTEXT("Column.GameVisibility.DefaultLabel", "Game Visibility"))
		.FixedWidth(IconWidth)
		.HeaderContent()
		[
			SNew(SImage)
			.Image(WorldBrowser::FWorldBrowserStyle::Get().GetBrush( "WorldBrowser.VisibleInGame" ))
			.ColorAndOpacity(FSlateColor::UseForeground())
			.ToolTipText(LOCTEXT("Column.GameVisibility.Tooltip", "Toggles the visibility in games"))
		]

		/** LevelName label column */
		+ SHeaderRow::Column( HierarchyColumns::ColumnID_LevelLabel )
		.FillWidth(1.f)
		.DefaultLabel(LOCTEXT("Column.Level.DefaultLabel", "Level"))
		.ShouldGenerateWidget(true) // This column cannot be toggled off by the user
		.HeaderContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("Column.Level.Label", "Level"))
			.ToolTipText(LOCTEXT("Column.Level.Tooltip", "Level"))
		]

		/** Lighting Scenario column */
		+ SHeaderRow::Column(HierarchyColumns::ColumnID_LightingScenario)
		.Visibility(bFoldersOnlyMode ? EVisibility::Collapsed : EVisibility::Visible)
		.DefaultLabel(LOCTEXT("Column.LightingScenario.DefaultLabel", "Lighting Scenario"))
		.FixedWidth(IconWidth)
		.HeaderContent()
		[
			SNew(SImage)
			.Image(FAppStyle::Get().GetBrush("Level.LightingScenarioIconSolid16x"))
			.ColorAndOpacity(FSlateColor::UseForeground())
			.ToolTipText(LOCTEXT("Column.LightingScenario.Tooltip", "Lighting Scenario"))
		]

		/** Level lock column */
		+SHeaderRow::Column(HierarchyColumns::ColumnID_Lock)
		.Visibility(bFoldersOnlyMode ? EVisibility::Collapsed : EVisibility::Visible)
		.DefaultLabel(LOCTEXT("Column.LevelLock.DefaultLabel", "Lock"))
		.FixedWidth(IconWidth)
		.HeaderContent()
		[
			SNew(SImage)
			.Image(FAppStyle::Get().GetBrush("Icons.Lock"))
			.ColorAndOpacity(FSlateColor::UseForeground())
			.ToolTipText(LOCTEXT("Column.LevelLock.Tooltip", "Lock"))
		]
	
		/** Level kismet column */
		+ SHeaderRow::Column(HierarchyColumns::ColumnID_Kismet)
		.Visibility(bFoldersOnlyMode ? EVisibility::Collapsed : EVisibility::Visible)
		.DefaultLabel(LOCTEXT("Column.Kismet.DefaultLabel", "Open Blueprint"))
		.FixedWidth(IconWidth)
		.HeaderContent()
		[
			SNew(SImage)
			.ColorAndOpacity(FSlateColor::UseForeground())
			.Image(FAppStyle::Get().GetBrush("Icons.Blueprints"))
			.ToolTipText(LOCTEXT("Column.Kismet.Tooltip", "Open the level blueprint for this Level"))
		]

		/** Level SCC status column */
		+ SHeaderRow::Column(HierarchyColumns::ColumnID_SCCStatus)
		.Visibility(bFoldersOnlyMode ? EVisibility::Collapsed : EVisibility::Visible)
		.DefaultLabel(LOCTEXT("Column.SCC.DefaultLabel", "Revision Control"))
		.FixedWidth(IconWidth)
		.HeaderContent()
		[
			SNew(SImage)
			.ColorAndOpacity(FSlateColor::UseForeground())
			.Image(FRevisionControlStyleManager::Get().GetBrush("RevisionControl.Icon"))
			.ToolTipText(LOCTEXT("Column.SCC.Tooltip", "Status in Revision Control"))
		]

		/** Level save column */
		+ SHeaderRow::Column(HierarchyColumns::ColumnID_Save)
		.Visibility(bFoldersOnlyMode ? EVisibility::Collapsed : EVisibility::Visible)
		.DefaultLabel(LOCTEXT("Column.Save.DefaultLabel", "Save"))
		.FixedWidth(IconWidth)
		.HeaderContent()
		[
			SNew(SImage)
			.ColorAndOpacity(FSlateColor::UseForeground())
			.Image(FAppStyle::Get().GetBrush("Icons.SaveModified"))
			.ToolTipText(LOCTEXT("Column.Save.Tooltip", "Save this Level"))
		]

		/** Level color column */
		+ SHeaderRow::Column(HierarchyColumns::ColumnID_Color)
		.Visibility(bFoldersOnlyMode ? EVisibility::Collapsed : EVisibility::Visible)
		.DefaultLabel(LOCTEXT("Column.Color.DefaultLabel", "Color"))
		.FixedWidth(IconWidth)
		.HeaderContent()
		[
			SNew(SImage)
			.ColorAndOpacity(FSlateColor(FColor::White))
			.Image(FAppStyle::Get().GetBrush("Level.ColorIcon"))
			.ToolTipText(LOCTEXT("Column.Color.Tooltip", "Color used for visualization of Level"))
		];
	
	UWorldBrowserConfig::Initialize();
	UWorldBrowserConfig* Config = UWorldBrowserConfig::Get();
	for (const TPair<FName, bool>& VisibilityPair : Config->ColumnConfig.ColumnVisibilities)
	{
		const FName ColumnId = VisibilityPair.Key;
		if (!IsRequiredColumn(ColumnId))
		{
			SetColumnVisible(ColumnId, VisibilityPair.Value);
		}
	}
	
	return HeaderRow;
}

void SWorldHierarchyImpl::SaveColumnVisibilitiesIntoConfig()
{
	if (bIsProgrammaticallyChangingColumnVisibility)
	{
		return;
	}
	
	UWorldBrowserConfig::Initialize();
	UWorldBrowserConfig* Config = UWorldBrowserConfig::Get();

	TMap<FName, bool>& Visibilities = Config->ColumnConfig.ColumnVisibilities;
	Visibilities.Empty();
	for (const SHeaderRow::FColumn& Column : HeaderRowWidget->GetColumns())
	{
		const FName ColumnId = Column.ColumnId;
		if (!IsRequiredColumn(ColumnId))
		{
			Visibilities.Add(ColumnId, IsColumnVisible(ColumnId));
		}
	}
	
	Config->SaveEditorConfig();
}

bool SWorldHierarchyImpl::IsRequiredColumn(FName ColumnId)
{
	return ColumnId == UE::WorldHierarchy::HierarchyColumns::ColumnID_LevelLabel;
}

bool SWorldHierarchyImpl::IsKnownColumn(FName ColumnId)
{
	using namespace UE::WorldHierarchy;
	return ColumnId == HierarchyColumns::ColumnID_EditorVisibility
		|| ColumnId == HierarchyColumns::ColumnID_GameVisibility
		|| ColumnId == HierarchyColumns::ColumnID_LevelLabel
		|| ColumnId == HierarchyColumns::ColumnID_LightingScenario
		|| ColumnId == HierarchyColumns::ColumnID_Lock
		|| ColumnId == HierarchyColumns::ColumnID_SCCStatus
		|| ColumnId == HierarchyColumns::ColumnID_Save
		|| ColumnId == HierarchyColumns::ColumnID_Color
		|| ColumnId == HierarchyColumns::ColumnID_Kismet;
}

TSharedRef<ITableRow> SWorldHierarchyImpl::GenerateTreeRow(WorldHierarchy::FWorldTreeItemPtr Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	check(Item.IsValid());

	return SNew(SWorldHierarchyItem, OwnerTable)
		.InWorldModel(WorldModel)
		.InHierarchy(SharedThis(this))
		.InItemModel(Item)
		.IsItemExpanded(Item->Flags.bExpanded)
		.HighlightText(this, &SWorldHierarchyImpl::GetSearchBoxText)
		.FoldersOnlyMode(bFoldersOnlyMode)
		;
}

void SWorldHierarchyImpl::GetChildrenForTree(WorldHierarchy::FWorldTreeItemPtr Item, TArray<WorldHierarchy::FWorldTreeItemPtr>& OutChildren)
{
	OutChildren = Item->GetChildren();

	if (Item->Flags.bChildrenRequiresSort)
	{
		if (OutChildren.Num() > 0)
		{
			SortItems(OutChildren);

			// Empty out the children and repopulate them in the correct order
			Item->RemoveAllChildren();

			for (WorldHierarchy::FWorldTreeItemPtr Child : OutChildren)
			{
				Item->AddChild(Child.ToSharedRef());
			}
		}

		Item->Flags.bChildrenRequiresSort = false;
	}
}

bool SWorldHierarchyImpl::PassesFilter(const WorldHierarchy::IWorldTreeItem& Item)
{
	bool bPassesFilter = true;

	WorldHierarchy::FFolderTreeItem* Folder = Item.GetAsFolderTreeItem();

	if (bFoldersOnlyMode && Folder == nullptr)
	{
		// Level items should fail to pass the filter if we only want to display folders
		bPassesFilter = false;
	}
	else
	{
		bPassesFilter = SearchBoxHierarchyFilter->PassesFilter(Item);
	}

	if (bPassesFilter && ExcludedFolders.Num() > 0)
	{
		if (Folder != nullptr)
		{
			FName CheckPath = Folder->GetFullPath();

			// Folders should not be shown if it or its parent have been excluded
			while (!CheckPath.IsNone())
			{
				if (ExcludedFolders.Contains(CheckPath))
				{
					bPassesFilter = false;
					break;
				}

				CheckPath = WorldHierarchy::GetParentPath(CheckPath);
			}
		}
	}

	return bPassesFilter;
}

TSharedPtr<SWidget> SWorldHierarchyImpl::ConstructLevelContextMenu() const
{
	TSharedRef<SWidget> MenuWidget = SNullWidget::NullWidget;

	if (!WorldModel->IsReadOnly())
	{
		UToolMenus* ToolMenus = UToolMenus::Get();
		static const FName MenuName = "WorldBrowser.WorldHierarchy.LevelContextMenu";
		if (!ToolMenus->IsMenuRegistered(MenuName))
		{
			ToolMenus->RegisterMenu(MenuName);
		}

		FToolMenuContext Context(WorldModel->GetCommandList(), TSharedPtr<FExtender>());
		UToolMenu* Menu = ToolMenus->GenerateMenu(MenuName, Context);

		TArray<WorldHierarchy::FWorldTreeItemPtr> SelectedItems = GetSelectedTreeItems();

		if (SelectedItems.Num() == 1)
		{
			// If exactly one item is selected, allow it to generate its own context menu
			SelectedItems[0]->GenerateContextMenu(Menu, *this);
		}
		else if (SelectedItems.Num() == 0)
		{
			// If no items are selected, allow the first root level item to create a context menu
			RootTreeItems[0]->GenerateContextMenu(Menu, *this);
		}

		Menu->AddDynamicSection("HierarchyDynamicSection", FNewToolMenuDelegateLegacy::CreateLambda([this](FMenuBuilder& MenuBuilder, UToolMenu* Menu)
		{
			const bool bIsGameVisibilityColumnVisible = IsColumnVisible(UE::WorldHierarchy::HierarchyColumns::ColumnID_GameVisibility);
			WorldModel->BuildHierarchyMenu(
				MenuBuilder,
				bIsGameVisibilityColumnVisible ? EBuildHierarchyMenuFlags::ShowGameVisibility : EBuildHierarchyMenuFlags::None
				);
		}));

		// Generate the "Move To" and "Select" submenus based on the current selection
		if (WorldModel->HasFolderSupport())
		{
			bool bOnlyFoldersSelected = SelectedItems.Num() > 0;
			bool bAllSelectedItemsCanMove = SelectedItems.Num() > 0;

			for (WorldHierarchy::FWorldTreeItemPtr Item : SelectedItems)
			{
				bOnlyFoldersSelected &= Item->GetAsFolderTreeItem() != nullptr;
				bAllSelectedItemsCanMove &= Item->CanChangeParents();

				if (!bOnlyFoldersSelected && !bAllSelectedItemsCanMove)
				{
					// Neither submenu can be built, kill the check
					break;
				}
			}

			if (bAllSelectedItemsCanMove && FLevelFolders::IsAvailable())
			{
				FToolMenuSection& Section = Menu->AddSection("Section");
				Section.AddSubMenu(
					"MoveSelectionTo",
					LOCTEXT("MoveSelectionTo", "Move To"),
					LOCTEXT("MoveSelectionTo_Tooltip", "Move selection to another folder"),
					FNewMenuDelegate::CreateSP(const_cast<SWorldHierarchyImpl*>(this), &SWorldHierarchyImpl::FillFoldersSubmenu)
				);
			}

			if (bOnlyFoldersSelected)
			{
				FToolMenuSection& Section = Menu->AddSection("Section");
				Section.AddSubMenu(
					"SelectSubmenu",
					LOCTEXT("SelectSubmenu", "Select"),
					LOCTEXT("SelectSubmenu_Tooltip", "Select child items of the current selection"),
					FNewMenuDelegate::CreateSP(const_cast<SWorldHierarchyImpl*>(this), &SWorldHierarchyImpl::FillSelectionSubmenu)
				);
			}
		}

		MenuWidget = ToolMenus->GenerateWidget(Menu);
	}

	return MenuWidget;
}

void SWorldHierarchyImpl::FillFoldersSubmenu(FMenuBuilder& MenuBuilder)
{
	TArray<WorldHierarchy::FWorldTreeItemPtr> SelectedItems = GetSelectedTreeItems();
	check(SelectedItems.Num() > 0);

	// Assume that the root item of the first selected item is the root for all of them
	TSharedPtr<FLevelModel> RootItem = SelectedItems[0]->GetRootItem();
	FName RootPath = NAME_None;

	MenuBuilder.AddMenuEntry(
		LOCTEXT("CreateNewFolder", "Create New Folder"),
		LOCTEXT("CreateNewFolder_Tooltip", "Move the selection to a new folder"),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "WorldBrowser.NewFolderIcon"),
		FExecuteAction::CreateSP(this, &SWorldHierarchyImpl::CreateFolder, RootItem, RootPath, /*bMoveSelected*/ true)
	);

	AddMoveToFolderOutliner(MenuBuilder, SelectedItems, RootItem.ToSharedRef());
}

void SWorldHierarchyImpl::AddMoveToFolderOutliner(FMenuBuilder& MenuBuilder, const TArray<WorldHierarchy::FWorldTreeItemPtr>& SelectedItems, TSharedRef<FLevelModel> RootItem)
{
	FLevelFolders& LevelFolders = FLevelFolders::Get();

	if (LevelFolders.GetFolderProperties(RootItem).Num() > 0)
	{
		TSet<FName> ExcludedFolderPaths;

		// Exclude selected folders
		for (WorldHierarchy::FWorldTreeItemPtr Item : SelectedItems)
		{
			if (WorldHierarchy::FFolderTreeItem* Folder = Item->GetAsFolderTreeItem())
			{
				ExcludedFolderPaths.Add(Folder->GetFullPath());
			}
		}

		// Copy the world model to ensure that any delegates fired for the mini hierarchy doesn't affect the main hierarchy
		FWorldBrowserModule& WorldBrowserModule = FModuleManager::LoadModuleChecked<FWorldBrowserModule>("WorldBrowser");
		TSharedPtr<FLevelCollectionModel> WorldModelCopy = WorldBrowserModule.SharedWorldModel(WorldModel->GetWorld());

		TSharedRef<SWidget> MiniHierarchy =
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.MaxHeight(400.0f)
			[
				SNew(SWorldHierarchyImpl)
				.InWorldModel(WorldModelCopy)
				.ShowFoldersOnly(true)
				.InExcludedFolders(ExcludedFolderPaths)
				.OnItemPickedDelegate(FOnWorldHierarchyItemPicked::CreateSP(this, &SWorldHierarchyImpl::MoveSelectionTo))
			];

		MenuBuilder.BeginSection(FName(), LOCTEXT("ExistingFolders", "Existing:"));
		MenuBuilder.AddWidget(MiniHierarchy, FText::GetEmpty(), false);
		MenuBuilder.EndSection();
	}
}

void SWorldHierarchyImpl::MoveSelectionTo(WorldHierarchy::FWorldTreeItemRef Item)
{
	FSlateApplication::Get().DismissAllMenus();

	TSharedPtr<FLevelModel> RootLevel = Item->GetRootItem();
	FName Path = NAME_None;

	if (WorldHierarchy::FFolderTreeItem* Folder = Item->GetAsFolderTreeItem())
	{
		Path = Folder->GetFullPath();
	}

	MoveItemsTo(RootLevel, Path);

	RefreshView();
}

void SWorldHierarchyImpl::FillSelectionSubmenu(FMenuBuilder& MenuBuilder)
{
	const bool bSelectAllDescendants = true;

	MenuBuilder.AddMenuEntry(
		LOCTEXT("SelectImmediateChildren", "Immediate Children"),
		LOCTEXT("SelectImmediateChildren_Tooltip", "Select all immediate children of the selected folders"),
		FSlateIcon(),
		FExecuteAction::CreateSP(this, &SWorldHierarchyImpl::SelectFolderDescendants, !bSelectAllDescendants)
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("SelectAllDescendants", "All Descendants"),
		LOCTEXT("SelectAllDescendants_Tooltip", "Selects all descendants of the selected folders"),
		FSlateIcon(),
		FExecuteAction::CreateSP(this, &SWorldHierarchyImpl::SelectFolderDescendants, bSelectAllDescendants)
	);
}

void SWorldHierarchyImpl::SelectFolderDescendants(bool bSelectAllDescendants)
{
	TArray<WorldHierarchy::FWorldTreeItemPtr> OldSelection = GetSelectedTreeItems();
	FLevelModelList SelectedLevels;

	TreeWidget->ClearSelection();

	for (WorldHierarchy::FWorldTreeItemPtr Item : OldSelection)
	{
		for (WorldHierarchy::FWorldTreeItemPtr Child : Item->GetChildren())
		{
			if (bSelectAllDescendants)
			{
				SelectedLevels.Append(Child->GetLevelModels());
			}
			else
			{
				SelectedLevels.Append(Child->GetModel());
			}
		}
	}

	if (SelectedLevels.Num() > 0)
	{
		WorldModel->SetSelectedLevels(SelectedLevels);
	}
}

void SWorldHierarchyImpl::MoveDroppedItems(const TArray<WorldHierarchy::FWorldTreeItemPtr>& DraggedItems, FName FolderPath)
{
	if (DraggedItems.Num() > 0)
	{
		// Ensure that the dragged items are selected in the tree
		TreeWidget->ClearSelection();

		for (WorldHierarchy::FWorldTreeItemPtr Item : DraggedItems)
		{
			TreeWidget->SetItemSelection(Item, true);
		}

		// Assume that the root of the first is the root of all the items
		const FScopedTransaction Transaction(LOCTEXT("ItemsMoved", "Move World Hierarchy Items"));
		MoveItemsTo(DraggedItems[0]->GetRootItem(), FolderPath);

		RefreshView();
	}
}

void SWorldHierarchyImpl::AddDroppedLevelsToFolder(const TArray<FAssetData>& WorldAssetList, FName FolderPath)
{
	if (WorldAssetList.Num() > 0)
	{
		// Populate the set of existing levels in the world
		TSet<FName> ExistingLevels;
		for (TSharedPtr<FLevelModel> Level : WorldModel->GetAllLevels())
		{
			ExistingLevels.Add(Level->GetLongPackageName());
		}

		WorldModel->AddExistingLevelsFromAssetData(WorldAssetList);

		// Set the folder path of any newly added levels
		for (TSharedPtr<FLevelModel> Level : WorldModel->GetAllLevels())
		{
			if (!ExistingLevels.Contains(Level->GetLongPackageName()))
			{
				Level->SetFolderPath(FolderPath);
			}
		}

		RefreshView();
	}
}

bool SWorldHierarchyImpl::IsColumnVisible(FName ColumnId) const
{
	return HeaderRowWidget->IsColumnVisible(ColumnId);
}

void SWorldHierarchyImpl::SetColumnVisible(FName ColumnId, bool bVisible)
{
	if (IsKnownColumn(ColumnId))
	{
		TGuardValue Guard(bIsProgrammaticallyChangingColumnVisibility, true);
		HeaderRowWidget->SetShowGeneratedColumn(ColumnId, bVisible);
	}
}

bool SWorldHierarchyImpl::IsVisibleInConfig(FName ColumnId)
{
	UWorldBrowserConfig::Initialize();
	UWorldBrowserConfig* Config = UWorldBrowserConfig::Get();
	const bool* bIsVisible = Config->ColumnConfig.ColumnVisibilities.Find(ColumnId);
	return IsRequiredColumn(ColumnId) || !bIsVisible || *bIsVisible;
}

void SWorldHierarchyImpl::SetWillBeVisibleInConfigTransient(FName ColumnId, bool bIsVisible)
{
	if (!IsRequiredColumn(ColumnId) && IsKnownColumn(ColumnId))
	{
		UWorldBrowserConfig::Initialize();
		UWorldBrowserConfig* Config = UWorldBrowserConfig::Get();
		Config->ColumnConfig.ColumnVisibilities.Add(ColumnId, bIsVisible);
	}
}

void SWorldHierarchyImpl::OnTreeItemScrolledIntoView( WorldHierarchy::FWorldTreeItemPtr Item, const TSharedPtr<ITableRow>& Widget )
{
	if (Item == ItemPendingRename)
	{
		ItemPendingRename = nullptr;
		Item->RenameRequestEvent.ExecuteIfBound();
	}
}

void SWorldHierarchyImpl::OnExpansionChanged(WorldHierarchy::FWorldTreeItemPtr Item, bool bIsItemExpanded)
{
	Item->SetExpansion(bIsItemExpanded);

	WorldHierarchy::FFolderTreeItem* Folder = Item->GetAsFolderTreeItem();
	if (FLevelFolders::IsAvailable() && Folder != nullptr)
	{
		if (FLevelFolderProps* Props = FLevelFolders::Get().GetFolderProperties(Item->GetRootItem().ToSharedRef(), Folder->GetFullPath()))
		{
			Props->bExpanded = Item->Flags.bExpanded;
		}
	}

	RefreshView();
}

void SWorldHierarchyImpl::OnSelectionChanged(const WorldHierarchy::FWorldTreeItemPtr Item, ESelectInfo::Type SelectInfo)
{
	if (bUpdatingSelection)
	{
		return;
	}

	bUpdatingSelection = true;

	TArray<WorldHierarchy::FWorldTreeItemPtr> SelectedItems = GetSelectedTreeItems();
	FLevelModelList SelectedLevels;

	for (const WorldHierarchy::FWorldTreeItemPtr& TreeItem : SelectedItems)
	{
		// Folder items should return all child models, but anything else should only return the model for that item
		if (TreeItem->GetAsFolderTreeItem() != nullptr)
		{
			SelectedLevels.Append(TreeItem->GetLevelModels());
		}
		else
		{
			SelectedLevels.Append(TreeItem->GetModel());
		}
	}

	if (!bFoldersOnlyMode)
	{
		WorldModel->SetSelectedLevels(SelectedLevels);
	}
	bUpdatingSelection = false;

	if (TreeWidget->GetNumItemsSelected() > 0)
	{
		OnItemPicked.ExecuteIfBound(GetSelectedTreeItems()[0].ToSharedRef());
	}
}

void SWorldHierarchyImpl::OnUpdateSelection()
{
	if (bUpdatingSelection)
	{
		return;
	}

	bUpdatingSelection = true;

	ItemsSelectedAfterRefresh.Empty();
	const FLevelModelList& SelectedItems = WorldModel->GetSelectedLevels();
	TreeWidget->ClearSelection();

	// To get the list of items that should be displayed as selected we need to find the level tree items belonging to the selected level models.
	if (SelectedItems.Num() > 0)
	{
		for (auto It = TreeItemMap.CreateConstIterator(); It; ++It)
		{
			WorldHierarchy::FWorldTreeItemPtr TreeItemPtr = It->Value;
			if (TreeItemPtr.IsValid())
			{
				for (auto SelectedItemIt = SelectedItems.CreateConstIterator(); SelectedItemIt; ++SelectedItemIt)
				{
					if (TreeItemPtr->HasModel(*SelectedItemIt))
					{
						ItemsSelectedAfterRefresh.Add(It->Key);
						break;
					}
				}
			}
		}
	}


	RefreshView();

	bUpdatingSelection = false;
}

void SWorldHierarchyImpl::OnTreeViewMouseButtonDoubleClick(WorldHierarchy::FWorldTreeItemPtr Item)
{
	if (Item->CanBeCurrent())
	{
		Item->MakeCurrent();
	}
	else
	{
		Item->SetExpansion(!Item->Flags.bExpanded);
		TreeWidget->SetItemExpansion(Item, Item->Flags.bExpanded);
	}
}

FReply SWorldHierarchyImpl::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (WorldModel->GetCommandList()->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	else if ( InKeyEvent.GetKey() == EKeys::F2 )
	{
		// If a single folder is selected, F2 should attempt to rename it
		if (TreeWidget->GetNumItemsSelected() == 1)
		{
			WorldHierarchy::FWorldTreeItemPtr ItemToRename = GetSelectedTreeItems()[0];

			if (ItemToRename->GetAsFolderTreeItem() != nullptr)
			{
				ItemPendingRename = ItemToRename;
				ScrollItemIntoView(ItemToRename);

				return FReply::Handled();
			}
		}
	}
	else if ( InKeyEvent.GetKey() == EKeys::Platform_Delete )
	{
		// Delete was pressed, but no levels were unloaded. Any selected folders should be removed transactionally
		const bool bTransactional = true;
		DeleteFolders(GetSelectedTreeItems(), bTransactional);
	}
	// F5 (Refresh) should be handled by the world model

	return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
}

void SWorldHierarchyImpl::OnBroadcastLevelsUnloaded()
{
	// We deleted levels from the hierarchy, so do not record the folder delete transaction either
	const bool bTransactional = false;
	DeleteFolders(GetSelectedTreeItems(), bTransactional);
}

void SWorldHierarchyImpl::InitiateRename(WorldHierarchy::FWorldTreeItemRef InItem)
{
	// Only folders items are valid for rename in this view
	if (InItem->GetAsFolderTreeItem() != nullptr)
	{
		ItemPendingRename = InItem;
		ScrollItemIntoView(InItem);
	}
}

void SWorldHierarchyImpl::EmptyTreeItems()
{
	for (auto& Pair : TreeItemMap)
	{
		Pair.Value->RemoveAllChildren();
	}

	PendingOperations.Empty();
	TreeItemMap.Reset();
	PendingTreeItemMap.Reset();

	RootTreeItems.Empty();
	NewItemActions.Empty();
	ItemPendingRename.Reset();
}

void SWorldHierarchyImpl::RepopulateEntireTree()
{
	EmptyTreeItems();

	for (const TSharedPtr<FLevelModel>& Level : WorldModel->GetFilteredLevels())
	{
		if (Level.IsValid())
		{
			ConstructItemFor<WorldHierarchy::FLevelModelTreeItem>(Level.ToSharedRef());
		}
	}

	if (FLevelFolders::IsAvailable() && WorldModel->HasFolderSupport())
	{
		FLevelFolders& LevelFolders = FLevelFolders::Get();

		// Add any folders which might match the search terms for each root level
		for (TSharedPtr<FLevelModel> RootLevel : WorldModel->GetRootLevelList())
		{
			for (const auto& Pair : LevelFolders.GetFolderProperties(RootLevel.ToSharedRef()))
			{
				if (!TreeItemMap.Contains(Pair.Key))
				{
					ConstructItemFor<WorldHierarchy::FFolderTreeItem>(Pair.Key);
				}
			}
		}
	}
}

TMap<WorldHierarchy::FWorldTreeItemID, bool> SWorldHierarchyImpl::GetParentsExpansionState() const
{
	TMap<WorldHierarchy::FWorldTreeItemID, bool> ExpansionStates;

	for (const auto& Pair : TreeItemMap)
	{
		if (Pair.Value->GetChildren().Num() > 0)
		{
			ExpansionStates.Add(Pair.Key, Pair.Value->Flags.bExpanded);
		}
	}

	return ExpansionStates;
}

void SWorldHierarchyImpl::SetParentsExpansionState(const TMap<WorldHierarchy::FWorldTreeItemID, bool>& ExpansionInfo)
{
	for (const auto& Pair : TreeItemMap)
	{
		auto& Item = Pair.Value;
		if (Item->GetChildren().Num() > 0)
		{
			const bool* bExpandedPtr = ExpansionInfo.Find(Pair.Key);
			bool bExpanded = bExpandedPtr != nullptr ? *bExpandedPtr : Item->Flags.bExpanded;

			TreeWidget->SetItemExpansion(Item, bExpanded);
		}
	}
}

void SWorldHierarchyImpl::OnBroadcastFolderCreate(TSharedPtr<FLevelModel> LevelModel, FName NewPath)
{
	if (!TreeItemMap.Contains(NewPath))
	{
		ConstructItemFor<WorldHierarchy::FFolderTreeItem>(NewPath);
	}
}

void SWorldHierarchyImpl::OnBroadcastFolderDelete(TSharedPtr<FLevelModel> LevelModel, FName Path)
{
	WorldHierarchy::FWorldTreeItemPtr* Folder = TreeItemMap.Find(Path);

	if (Folder != nullptr)
	{
		PendingOperations.Emplace(WorldHierarchy::FPendingWorldTreeOperation::Removed, Folder->ToSharedRef());
		RefreshView();
	}
}

void SWorldHierarchyImpl::OnBroadcastFolderMove(TSharedPtr<FLevelModel> LevelModel, FName OldPath, FName NewPath)
{
	WorldHierarchy::FWorldTreeItemPtr Folder = TreeItemMap.FindRef(OldPath);

	if (Folder.IsValid())
	{
		// Remove the item with the old ID
		TreeItemMap.Remove(Folder->GetID());

		// Get all items that were moved
		TArray<WorldHierarchy::FWorldTreeItemPtr> AllSelectedItems = GetSelectedTreeItems();

		// Change the path, and place it back in the tree with the new ID
		{
			WorldHierarchy::FFolderTreeItem* FolderItem = Folder->GetAsFolderTreeItem();
			FolderItem->SetNewPath(NewPath);
		}

		for (WorldHierarchy::FWorldTreeItemPtr Child : Folder->GetChildren())
		{
			// Any level model children that were not explicitly moved will need to be moved here to remain in
			// sync with their parent folders
			if (!AllSelectedItems.Contains(Child) && Child->GetAsLevelModelTreeItem() != nullptr)
			{
				Child->SetParentPath(NewPath);
			}
		}

		TreeItemMap.Add(Folder->GetID(), Folder);

		PendingOperations.Emplace(WorldHierarchy::FPendingWorldTreeOperation::Moved, Folder.ToSharedRef());
		RefreshView();
	}
}

void SWorldHierarchyImpl::FullRefresh()
{
	bFullRefresh = true;
	RefreshView();
}

void SWorldHierarchyImpl::RebuildFoldersAndFullRefresh()
{
	bRebuildFolders = true;
	FullRefresh();
}

void SWorldHierarchyImpl::RequestSort()
{
	bSortDirty = true;
}

void SWorldHierarchyImpl::Populate()
{
	TGuardValue<bool> ReentrantGuard(bIsReentrant, true);

	bool bMadeSignificantChanges = false;

	const TMap<WorldHierarchy::FWorldTreeItemID, bool> ExpansionStateInfo = GetParentsExpansionState();

	if (bRebuildFolders)
	{
		if (FLevelFolders::IsAvailable())
		{
			FLevelFolders& LevelFolders = FLevelFolders::Get();

			for (TSharedPtr<FLevelModel> LevelModel : WorldModel->GetRootLevelList())
			{
				LevelFolders.RebuildFolderList(LevelModel.ToSharedRef());
			}
		}

		bRebuildFolders = false;
	}

	if (bFullRefresh)
	{
		RepopulateEntireTree();

		bFullRefresh = false;
		bMadeSignificantChanges = true;
	}

	if (PendingOperations.Num() > 0)
	{
		const int32 End = FMath::Min(PendingOperations.Num(), MaxPendingOperations);
		for (int32 Index = 0; Index < End; ++Index)
		{
			const WorldHierarchy::FPendingWorldTreeOperation& PendingOp = PendingOperations[Index];
			switch (PendingOp.Operation)
			{
			case WorldHierarchy::FPendingWorldTreeOperation::Added:
				bMadeSignificantChanges = AddItemToTree(PendingOp.Item);
				break;

			case WorldHierarchy::FPendingWorldTreeOperation::Moved:
				bMadeSignificantChanges = true;
				OnItemMoved(PendingOp.Item);
				break;

			case WorldHierarchy::FPendingWorldTreeOperation::Removed:
				bMadeSignificantChanges = true;
				RemoveItemFromTree(PendingOp.Item);
				break;

			default:
				check(false);
				break;
			}
		}

		PendingOperations.RemoveAt(0, End);
	}

	SetParentsExpansionState(ExpansionStateInfo);

	if (ItemsSelectedAfterRefresh.Num() > 0)
	{
		bool bScrolledIntoView = false;
		for (const WorldHierarchy::FWorldTreeItemID& ID : ItemsSelectedAfterRefresh)
		{
			if (TreeItemMap.Contains(ID))
			{
				WorldHierarchy::FWorldTreeItemPtr Item = TreeItemMap[ID];

				for (WorldHierarchy::FWorldTreeItemPtr ItemParent = Item->GetParent(); ItemParent.IsValid(); ItemParent = ItemParent->GetParent())
				{
					TreeWidget->SetItemExpansion(ItemParent, true);
				}

				TreeWidget->SetItemSelection(Item, true);

				if (!bScrolledIntoView)
				{
					bScrolledIntoView = true;
					TreeWidget->RequestScrollIntoView(Item);
				}
			}
		}

		ItemsSelectedAfterRefresh.Empty();
	}

	if (bMadeSignificantChanges)
	{
		RequestSort();
	}

	TreeWidget->RequestTreeRefresh();

	if (PendingOperations.Num() == 0)
	{
		NewItemActions.Empty();
		bNeedsRefresh = false;
	}
}

bool SWorldHierarchyImpl::AddItemToTree(WorldHierarchy::FWorldTreeItemRef InItem)
{
	const WorldHierarchy::FWorldTreeItemID ItemID = InItem->GetID();

	bool bItemAdded = false;

	PendingTreeItemMap.Remove(ItemID);
	if (!TreeItemMap.Find(ItemID))
	{
		// Not currently in the tree, check if the item passes the current filter

		bool bFilteredOut = !PassesFilter(*InItem);

		InItem->Flags.bFilteredOut = bFilteredOut;

		if (!bFilteredOut)
		{
			AddUnfilteredItemToTree(InItem);
			bItemAdded = true;

			if (WorldHierarchy::ENewItemAction* ActionPtr = NewItemActions.Find(ItemID))
			{
				WorldHierarchy::ENewItemAction Actions = *ActionPtr;

				if ((Actions & WorldHierarchy::ENewItemAction::Select) != WorldHierarchy::ENewItemAction::None)
				{
					TreeWidget->ClearSelection();
					TreeWidget->SetItemSelection(InItem, true);
				}

				if ((Actions & WorldHierarchy::ENewItemAction::Rename) != WorldHierarchy::ENewItemAction::None)
				{
					ItemPendingRename = InItem;
				}

				WorldHierarchy::ENewItemAction ScrollIntoView = WorldHierarchy::ENewItemAction::ScrollIntoView | WorldHierarchy::ENewItemAction::Rename;
				if ((Actions & ScrollIntoView) != WorldHierarchy::ENewItemAction::None)
				{
					ScrollItemIntoView(InItem);
				}
			}
		}
	}

	return bItemAdded;
}

void SWorldHierarchyImpl::AddUnfilteredItemToTree(WorldHierarchy::FWorldTreeItemRef InItem)
{
	WorldHierarchy::FWorldTreeItemPtr Parent = EnsureParentForItem(InItem);
	const WorldHierarchy::FWorldTreeItemID ItemID = InItem->GetID();

	if (TreeItemMap.Contains(ItemID))
	{
		UE_LOGF(LogWorldHierarchy, Error, "(%d | %ls) already exists in the World Hierarchy. Dumping map...", GetTypeHash(ItemID), *InItem->GetDisplayString());

		for (const auto& Entry : TreeItemMap)
		{
			UE_LOGF(LogWorldHierarchy, Log, "(%d | %ls)", GetTypeHash(Entry.Key), *Entry.Value->GetDisplayString());
		}

		// Treat this as a fatal error
		check(false);
	}

	TreeItemMap.Add(ItemID, InItem);

	if (Parent.IsValid())
	{
		Parent->AddChild(InItem);
	}
	else
	{
		RootTreeItems.Add(InItem);
	}

	if (FLevelFolders::IsAvailable())
	{
		WorldHierarchy::FFolderTreeItem* Folder = InItem->GetAsFolderTreeItem();

		if (Folder != nullptr)
		{
			if (const FLevelFolderProps* Props = FLevelFolders::Get().GetFolderProperties(InItem->GetRootItem().ToSharedRef(), Folder->GetFullPath()))
			{
				InItem->SetExpansion(Props->bExpanded);
			}
		}
	}
}

void SWorldHierarchyImpl::RemoveItemFromTree(WorldHierarchy::FWorldTreeItemRef InItem)
{
	if (TreeItemMap.Contains(InItem->GetID()))
	{
		WorldHierarchy::FWorldTreeItemPtr Parent = InItem->GetParent();

		if (Parent.IsValid())
		{
			Parent->RemoveChild(InItem);
			OnChildRemovedFromParent(Parent.ToSharedRef());
		}
		else
		{
			RootTreeItems.Remove(InItem);
		}

		TreeItemMap.Remove(InItem->GetID());
	}
}

void SWorldHierarchyImpl::OnItemMoved(WorldHierarchy::FWorldTreeItemRef InItem)
{
	// If the item no longer matches the filter, remove it from the tree
	if (!InItem->Flags.bFilteredOut && !PassesFilter(*InItem))
	{
		RemoveItemFromTree(InItem);
	}
	else
	{
		WorldHierarchy::FWorldTreeItemPtr Parent = InItem->GetParent();

		if (Parent.IsValid())
		{
			Parent->RemoveChild(InItem);
			OnChildRemovedFromParent(Parent.ToSharedRef());
		}
		else
		{
			RootTreeItems.Remove(InItem);
		}

		Parent = EnsureParentForItem(InItem);
		if (Parent.IsValid())
		{
			Parent->AddChild(InItem);
			Parent->SetExpansion(true);
			TreeWidget->SetItemExpansion(Parent, true);
		}
		else
		{
			RootTreeItems.Add(InItem);
		}
	}
}

void SWorldHierarchyImpl::ScrollItemIntoView(WorldHierarchy::FWorldTreeItemPtr Item)
{
	WorldHierarchy::FWorldTreeItemPtr Parent = Item->GetParent();

	while (Parent.IsValid())
	{
		TreeWidget->SetItemExpansion(Parent, true);
		Parent = Parent->GetParent();
	}

	TreeWidget->RequestScrollIntoView(Item);
}

void SWorldHierarchyImpl::OnChildRemovedFromParent(WorldHierarchy::FWorldTreeItemRef InParent)
{
	if (InParent->Flags.bFilteredOut && InParent->GetChildren().Num() == 0)
	{
		// Parent does not match the search terms nor does it have any children that matches the search terms
		RemoveItemFromTree(InParent);
	}
}

WorldHierarchy::FWorldTreeItemPtr SWorldHierarchyImpl::EnsureParentForItem(WorldHierarchy::FWorldTreeItemRef Item)
{
	WorldHierarchy::FWorldTreeItemID ParentID = Item->GetParentID();
	WorldHierarchy::FWorldTreeItemPtr ParentPtr;

	if (TreeItemMap.Contains(ParentID))
	{
		ParentPtr = TreeItemMap[ParentID];
	}
	else
	{
		ParentPtr = Item->CreateParent();
		if (ParentPtr.IsValid())
		{
			AddUnfilteredItemToTree(ParentPtr.ToSharedRef());
		}
	}


	return ParentPtr;
}

bool SWorldHierarchyImpl::IsTreeItemExpanded(WorldHierarchy::FWorldTreeItemPtr Item) const
{
	return Item->Flags.bExpanded;
}

void SWorldHierarchyImpl::SortItems(TArray<WorldHierarchy::FWorldTreeItemPtr>& Items)
{
	if (Items.Num() > 1)
	{
		Items.Sort([](WorldHierarchy::FWorldTreeItemPtr Item1, WorldHierarchy::FWorldTreeItemPtr Item2) {
			const int32 Priority1 = Item1->GetSortPriority();
			const int32 Priority2 = Item2->GetSortPriority();

			if (Priority1 == Priority2)
			{
				return Item1->GetDisplayString() < Item2->GetDisplayString();
			}

			return Priority1 > Priority2;
		});
	}
}

void SWorldHierarchyImpl::TransformLevelToString(const FLevelModel* Level, TArray<FString>& OutSearchStrings) const
{
	if (Level != nullptr && Level->HasValidPackage())
	{
		OutSearchStrings.Add(FPackageName::GetShortName(Level->GetLongPackageName()));
	}
}

void SWorldHierarchyImpl::TransformItemToString(const WorldHierarchy::IWorldTreeItem& Item, TArray<FString>& OutSearchStrings) const
{
	OutSearchStrings.Add(Item.GetDisplayString());
}

void SWorldHierarchyImpl::SetFilterText(const FText& InFilterText)
{
	// Ensure that the level and hierarchy filters remain in sync
	if (SearchBoxLevelFilter.IsValid())
	{
		SearchBoxLevelFilter->SetRawFilterText(InFilterText);
	}
	SearchBoxHierarchyFilter->SetRawFilterText(InFilterText);
}

FText SWorldHierarchyImpl::GetSearchBoxText() const
{
	return SearchBoxHierarchyFilter->GetRawFilterText();
}

FText SWorldHierarchyImpl::GetFilterStatusText() const
{
	const int32 SelectedLevelsCount = WorldModel->GetSelectedLevels().Num();
	const int32 TotalLevelsCount = WorldModel->GetAllLevels().Num();
	const int32 FilteredLevelsCount = WorldModel->GetFilteredLevels().Num();

	if (!WorldModel->IsFilterActive())
	{
		if (SelectedLevelsCount == 0)
		{
			return FText::Format(LOCTEXT("ShowingAllLevelsFmt", "{0} levels"), FText::AsNumber(TotalLevelsCount));
		}
		else
		{
			return FText::Format(LOCTEXT("ShowingAllLevelsSelectedFmt", "{0} levels ({1} selected)"), FText::AsNumber(TotalLevelsCount), FText::AsNumber(SelectedLevelsCount));
		}
	}
	else if (WorldModel->IsFilterActive() && FilteredLevelsCount == 0)
	{
		return FText::Format(LOCTEXT("ShowingNoLevelsFmt", "No matching levels ({0} total)"), FText::AsNumber(TotalLevelsCount));
	}
	else if (SelectedLevelsCount != 0)
	{
		return FText::Format(LOCTEXT("ShowingOnlySomeLevelsSelectedFmt", "Showing {0} of {1} levels ({2} selected)"), FText::AsNumber(FilteredLevelsCount), FText::AsNumber(TotalLevelsCount), FText::AsNumber(SelectedLevelsCount));
	}
	else
	{
		return FText::Format(LOCTEXT("ShowingOnlySomeLevelsFmt", "Showing {0} of {1} levels"), FText::AsNumber(FilteredLevelsCount), FText::AsNumber(TotalLevelsCount));
	}
}

FReply SWorldHierarchyImpl::OnCreateFolderClicked()
{
	// Assume that the folder will be created for the first persistent level
	TSharedPtr<FLevelModel> PersistentLevel = WorldModel->GetRootLevelList()[0];

	CreateFolder(PersistentLevel);

	return FReply::Handled();
}

EVisibility SWorldHierarchyImpl::GetEmptyLabelVisibility() const
{
	return ( !bFoldersOnlyMode || RootTreeItems.Num() > 0 ) ? EVisibility::Collapsed : EVisibility::Visible;
}

void SWorldHierarchyImpl::CreateFolder(TSharedPtr<FLevelModel> InModel, FName ParentPath /* = NAME_None */, const bool bMoveSelected)
{
	if (FLevelFolders::IsAvailable())
	{
		TSharedPtr<FLevelModel> PersistentLevelModel = InModel;
		if (!InModel.IsValid())
		{
			// We're not making this for any specific level...assume it's the first persistent level in the world
			PersistentLevelModel = WorldModel->GetRootLevelList()[0];
		}
	
		const FScopedTransaction Transaction(LOCTEXT("UndoAction_CreateFolder", "Create Folder"));

		FLevelFolders& LevelFolders = FLevelFolders::Get();
		FName NewFolderName = ParentPath;

		// Get the folder name for the selected level items
		if (NewFolderName.IsNone())
		{
			// Attempt to find the most relevant shared folder for all selected items
			TArray<WorldHierarchy::FWorldTreeItemPtr> SelectedItems = GetSelectedTreeItems();

			TSet<FName> SharedAncestorPaths = SelectedItems.Num() > 0 ? SelectedItems[0]->GetAncestorPaths() : TSet<FName>();

			for (int32 Index = 1; Index < SelectedItems.Num(); ++Index)
			{
				SharedAncestorPaths = SharedAncestorPaths.Intersect(SelectedItems[Index]->GetAncestorPaths());

				if (SharedAncestorPaths.Num() == 0)
				{
					// No common ancestor path found, put them at the root
					break;
				}
			}

			// Find the longest name in the shared ancestor paths, because that's the most local "root" folder
			for (FName Ancestor : SharedAncestorPaths)
			{
				if (Ancestor.ToString().Len() > NewFolderName.ToString().Len())
				{
					NewFolderName = Ancestor;
				}
			}
		}

		NewFolderName = LevelFolders.GetDefaultFolderName(PersistentLevelModel.ToSharedRef(), NewFolderName);

		if (bMoveSelected)
		{
			MoveItemsTo(PersistentLevelModel, NewFolderName);
		}
		else if (FLevelFolders::IsAvailable())
		{
			LevelFolders.CreateFolder(PersistentLevelModel.ToSharedRef(), NewFolderName);
			NewItemActions.Add(NewFolderName, WorldHierarchy::ENewItemAction::Select | WorldHierarchy::ENewItemAction::Rename);
		}
	}
}

void SWorldHierarchyImpl::MoveItemsTo(TSharedPtr<FLevelModel> InModel, FName InPath)
{
	if (FLevelFolders::IsAvailable())
	{
		FLevelFolders& LevelFolders = FLevelFolders::Get();

		// Get the selected folders first before any items move
		TArray<WorldHierarchy::FWorldTreeItemPtr> PreviouslySelectedItems = GetSelectedTreeItems();
		TArray<WorldHierarchy::FFolderTreeItem*> SelectedFolders;

		for (WorldHierarchy::FWorldTreeItemPtr Item : PreviouslySelectedItems)
		{
			if (WorldHierarchy::FFolderTreeItem* Folder = Item->GetAsFolderTreeItem())
			{
				SelectedFolders.Add(Folder);
			}
		}

		// Move the levels first
		LevelFolders.CreateFolderContainingSelectedLevels(WorldModel.ToSharedRef(), InModel.ToSharedRef(), InPath);

		// Ensure that any moved levels will have their hierarchy items updated
		for (TSharedPtr<FLevelModel> SelectedLevel : WorldModel->GetSelectedLevels())
		{
			WorldHierarchy::FWorldTreeItemID LevelID(SelectedLevel->GetLevelObject());

			if (TreeItemMap.Contains(LevelID))
			{
				PendingOperations.Emplace(WorldHierarchy::FPendingWorldTreeOperation::Moved, TreeItemMap[LevelID].ToSharedRef());
			}
		}

		// Move any of the previously selected folders 
		for (WorldHierarchy::FFolderTreeItem* Folder : SelectedFolders)
		{
			FName OldPath = Folder->GetFullPath();
			FName NewPath = FName(*(InPath.ToString() / Folder->GetLeafName().ToString()));

			LevelFolders.RenameFolder(Folder->GetRootItem().ToSharedRef(), OldPath, NewPath);
		}

		NewItemActions.Add(InPath, WorldHierarchy::ENewItemAction::Select | WorldHierarchy::ENewItemAction::Rename);
	}
}

void SWorldHierarchyImpl::DeleteFolders(TArray<WorldHierarchy::FWorldTreeItemPtr> SelectedItems, bool bTransactional/* = true*/)
{
	TArray<WorldHierarchy::FWorldTreeItemPtr> FolderItems;
	TSet<FName> DeletedPaths;
	
	for (WorldHierarchy::FWorldTreeItemPtr Item : SelectedItems)
	{
		// Only take folder items
		if (WorldHierarchy::FFolderTreeItem* Folder = Item->GetAsFolderTreeItem())
		{
			FolderItems.Add(Item);
			DeletedPaths.Add(Folder->GetFullPath());
		}
	}

	FScopedTransaction Transaction(LOCTEXT("DeleteFolderTransaction", "Delete Folder"));
	FLevelFolders& LevelFolders = FLevelFolders::Get();

	// Folders are deleted one at a time
	for (WorldHierarchy::FWorldTreeItemPtr Item : FolderItems)
	{
		TSharedRef<FLevelModel> LevelModel = Item->GetRootItem().ToSharedRef();

		// First, move the folder's children up to the ancestor that will not be deleted
		FName ItemPath = Item->GetAsFolderTreeItem()->GetFullPath();

		FName ParentPath = ItemPath;
		do 
		{ 
			ParentPath = WorldHierarchy::GetParentPath(ParentPath); 
		} while (DeletedPaths.Contains(ParentPath) && !ParentPath.IsNone());

		TArray<WorldHierarchy::FWorldTreeItemPtr> Children = Item->GetChildren();
		for (WorldHierarchy::FWorldTreeItemPtr Child : Children)
		{
			if (!SelectedItems.Contains(Child))
			{
				if (WorldHierarchy::FFolderTreeItem* ChildFolder = Child->GetAsFolderTreeItem())
				{
					FName NewChildPath = ChildFolder->GetLeafName();
					if (!ParentPath.IsNone())
					{
						NewChildPath = FName(*(ParentPath.ToString() / NewChildPath.ToString()));
					}

					LevelFolders.RenameFolder(LevelModel, ChildFolder->GetFullPath(), NewChildPath);
				}
				else
				{
					Child->SetParentPath(ParentPath);
					OnItemMoved(Child.ToSharedRef());
				}
			}
		}

		// Then delete the folder
		LevelFolders.DeleteFolder(LevelModel, ItemPath);
	}

	if (!bTransactional || FolderItems.Num() == 0)
	{
		Transaction.Cancel();
	}
}

FSlateColor SWorldHierarchyImpl::GetFilterStatusTextColor() const
{
	if (!WorldModel->IsFilterActive())
	{
		// White = no text filter
		return FLinearColor(1.0f, 1.0f, 1.0f);
	}
	else if (WorldModel->GetFilteredLevels().Num() == 0)
	{
		// Red = no matching actors
		return FLinearColor(1.0f, 0.4f, 0.4f);
	}
	else
	{
		// Green = found at least one match!
		return FLinearColor(0.4f, 1.0f, 0.4f);
	}
}

TSharedRef<SWidget> SWorldHierarchyImpl::GetViewButtonContent()
{
	FMenuBuilder MenuBuilder(true, NULL);

	MenuBuilder.BeginSection("SubLevelsViewMenu", LOCTEXT("ShowHeading", "Show"));
	{
		MenuBuilder.AddMenuEntry(LOCTEXT("ToggleDisplayPaths", "Display Paths"),
			LOCTEXT("ToggleDisplayPaths_Tooltip", "If enabled, displays the path for each level"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SWorldHierarchyImpl::ToggleDisplayPaths_Executed),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SWorldHierarchyImpl::GetDisplayPathsState)),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		MenuBuilder.AddMenuEntry(LOCTEXT("ToggleDisplayActorsCount", "Display Actors Count"),
			LOCTEXT("ToggleDisplayActorsCount_Tooltip", "If enabled, displays actors count for each level"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SWorldHierarchyImpl::ToggleDisplayActorsCount_Executed),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(this, &SWorldHierarchyImpl::GetDisplayActorsCountState)),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

bool SWorldHierarchyImpl::GetDisplayPathsState() const
{
	return WorldModel->GetDisplayPathsState();
}

void SWorldHierarchyImpl::ToggleDisplayActorsCount_Executed()
{
	WorldModel->SetDisplayActorsCountState(!WorldModel->GetDisplayActorsCountState());
}

bool SWorldHierarchyImpl::GetDisplayActorsCountState() const
{
	return WorldModel->GetDisplayActorsCountState();
}

void SWorldHierarchyImpl::PostUndo(bool bSuccess)
{
	if (!bIsReentrant)
	{
		FullRefresh();
	}
}

#undef LOCTEXT_NAMESPACE