// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "EditorUndoClient.h"

#include "LevelModel.h"
#include "LevelCollectionModel.h"
#include "Misc/TextFilter.h"
#include "SLevelsTreeWidget.h"
#include "UObject/ObjectSaveContext.h"

#include "IWorldTreeItem.h"

typedef TTextFilter<const WorldHierarchy::IWorldTreeItem& > HierarchyFilter;
typedef TTextFilter<const FLevelModel* > LevelTextFilter;

namespace WorldHierarchy
{
	/** Structure that defines an operation that should be applied to the world hierarchy */
	struct FPendingWorldTreeOperation
	{
		enum EType { Added, Removed, Moved, };
		FPendingWorldTreeOperation(EType InOp, FWorldTreeItemRef InItem) : Operation(InOp), Item(InItem) {}

		/** The type of operation to perform */
		EType Operation;

		/** The item affected by this operation */
		FWorldTreeItemRef Item;
	};

	/** Defines operations for items to perform when they are added to the tree */
	enum class ENewItemAction : uint8
	{
		None			= 0,		/** No Actions for the item */
		Select			= 1 << 0,	/** Item should be selected */
		ScrollIntoView	= 1 << 1,	/** Item should be selected */
		Rename			= 1 << 2,	/** Item should be renamed */
	};

	ENUM_CLASS_FLAGS(ENewItemAction)
}	// namespace WorldHierarchy

DECLARE_DELEGATE_OneParam(FOnWorldHierarchyItemPicked, WorldHierarchy::FWorldTreeItemRef);

class SWorldHierarchyImpl 
	: public SCompoundWidget
	, public FEditorUndoClient
{
public:
	SLATE_BEGIN_ARGS(SWorldHierarchyImpl)
		: _ShowFoldersOnly(false)
		{}
		/** The world represented by this hierarchy */
		SLATE_ARGUMENT(TSharedPtr<FLevelCollectionModel>, InWorldModel)
		/** If true, the hierarchy will only show folders for the world model */
		SLATE_ARGUMENT(bool, ShowFoldersOnly)
		/** If folders only mode is activated, this prevents certain folders from being displayed */
		SLATE_ARGUMENT(TSet<FName>, InExcludedFolders)
		/** A delegate to fire when an item is picked */
		SLATE_ARGUMENT(FOnWorldHierarchyItemPicked, OnItemPickedDelegate)
	SLATE_END_ARGS()

	SWorldHierarchyImpl();
	~SWorldHierarchyImpl();
	
	void Construct(const FArguments& InArgs);

	/** Regenerates current items */
	void RefreshView();

	virtual void Tick( const FGeometry& AllotedGeometry, const double InCurrentTime, const float InDeltaTime ) override;

	/** Creates a new folder for the hierarchy. If ParentPath is defined, the folder will be created relative to that path. */
	void CreateFolder(TSharedPtr<FLevelModel> InModel, FName ParentPath = NAME_None, const bool bMoveSelected = true);

	/** Moves the current selection to the specified path */
	void MoveItemsTo(TSharedPtr<FLevelModel> InModel, FName Path);

	/** Initiates a rename of the selected item */
	void InitiateRename(WorldHierarchy::FWorldTreeItemRef InItem);

	/** Deletes the folders contained in the selection from the hierarchy tree.  */
	void DeleteFolders(TArray<WorldHierarchy::FWorldTreeItemPtr> SelectedItems, bool bTransactional = true);

	/** Moves selected items from a drag and drop operation */
	void MoveDroppedItems(const TArray<WorldHierarchy::FWorldTreeItemPtr>& DraggedItems, FName FolderPath);

	/** Adds the specified levels to the hierarchy under the specified folder path */
	void AddDroppedLevelsToFolder(const TArray<FAssetData>& WorldAssetList, FName FolderPath);

	/** Helper funciton to get the selected items from the tree widget */
	TArray<WorldHierarchy::FWorldTreeItemPtr> GetSelectedTreeItems() const { return TreeWidget->GetSelectedItems(); }

	/**
	 * @see WorldHierarchyColumns.h for named columns.
	 * @return Whether Column is visible in the UI.
	 */
	bool IsColumnVisible(FName ColumnId) const;
	/**
	 * Sets whether Column is visible in the UI.
	 * @see WorldHierarchyColumns.h for named columns.
	 */
	void SetColumnVisible(FName ColumnId, bool bVisible);

	/** @return Whether the column will be visible according to the config. */
	static bool IsVisibleInConfig(FName ColumnId);
	/** Sets whether the next widget created from now will have ColumnId visible but does not save it into the config. */
	static void SetWillBeVisibleInConfigTransient(FName ColumnId, bool bIsVisible);

public:
	//~ FEditorUndoClient
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }

private:
	// The maximum number of pending operations to process at one time
	static const int32 MaxPendingOperations = 500;

	/** Creates the header row for the level hierarchy. */
	TSharedRef<SHeaderRow> CreateHeaderRow();

	/** Invokes when the column visibility changes. */
	void SaveColumnVisibilitiesIntoConfig();

	/** @return Whether the given column is always shown. */
	static bool IsRequiredColumn(FName ColumnId);
	
	/** @return Whether the column is used by this UI */
	static bool IsKnownColumn(FName ColumnId);
	
	/** Creates an item for the tree view */
	TSharedRef<ITableRow> GenerateTreeRow(WorldHierarchy::FWorldTreeItemPtr Item, const TSharedRef<STableViewBase>& OwnerTable);

	/** Handler for returning a list of children associated with a particular tree node */
	void GetChildrenForTree(WorldHierarchy::FWorldTreeItemPtr Item, TArray<WorldHierarchy::FWorldTreeItemPtr>& OutChildren);

	/** @return the SWidget containing the context menu */
	TSharedPtr<SWidget> ConstructLevelContextMenu() const;

	/** Builds the submenu for moving items to other folders in the hierarchy */
	void FillFoldersSubmenu(FMenuBuilder& MenuBuilder);

	/** Builds the list of valid folders that an item can be moved to */
	void AddMoveToFolderOutliner(FMenuBuilder& MenuBuilder, const TArray<WorldHierarchy::FWorldTreeItemPtr>& CachedSelectedItems, TSharedRef<FLevelModel> RootLevel);

	/** Builds the menu that allows for children of a folder to be selected from the context menu */
	void FillSelectionSubmenu(FMenuBuilder& MenuBuilder);

	/** Selects the level descendants of the currently selected folders. If bSelectAllDescendants is true, all descendants will be selected instead of immediate children */
	void SelectFolderDescendants(bool bSelectAllDescendants);

	/** Called after a tree item has been scrolled into view. This initiates a pending rename. */
	void OnTreeItemScrolledIntoView( WorldHierarchy::FWorldTreeItemPtr Item, const TSharedPtr<ITableRow>& Widget );

	/** Called by TreeView widget whenever tree item expanded or collapsed */
	void OnExpansionChanged(WorldHierarchy::FWorldTreeItemPtr Item, bool bIsItemExpanded);

	/** Called by TreeView widget whenever selection is changed */
	void OnSelectionChanged(const WorldHierarchy::FWorldTreeItemPtr Item, ESelectInfo::Type SelectInfo);

	/** Handles selection changes in data source */
	void OnUpdateSelection();

	/** 
	 *	Called by STreeView when the user double-clicks on an item 
	 *
	 *	@param	Item	The item that was double clicked
	 */
	void OnTreeViewMouseButtonDoubleClick(WorldHierarchy::FWorldTreeItemPtr Item);

	/**
	 * Checks to see if this widget supports keyboard focus.  Override this in derived classes.
	 *
	 * @return  True if this widget can take keyboard focus
	 */
	virtual bool SupportsKeyboardFocus() const override
	{
		return true;
	}

	/**
	 * Called after a key is pressed when this widget has focus
	 *
	 * @param MyGeometry The Geometry of the widget receiving the event
	 * @param  InKeyEvent  Key event
	 *
	 * @return  Returns whether the event was handled, along with other possible actions
	 */
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	/** Adds a new item for the specified type and refreshes the tree, provided it matches the filter terms */
	template <typename TreeItemType, typename DataType>
	void ConstructItemFor(const DataType& Data)
	{
		const TreeItemType Temporary(Data);
		bool bPassesFilter = PassesFilter(Temporary);

		if (bPassesFilter)
		{
			WorldHierarchy::FWorldTreeItemRef NewItem = MakeShareable(new TreeItemType(Data));
			NewItem->WorldModel = WorldModel;

			PendingOperations.Emplace(WorldHierarchy::FPendingWorldTreeOperation::Added, NewItem);
			PendingTreeItemMap.Add(NewItem->GetID(), NewItem);
			RefreshView();
		}
	}

	/** Applies incremental changes to the tree, or completely repopulates it */
	void Populate();

	/** Empties all items within the tree */
	void EmptyTreeItems();

	/** Fully rebuilds the tree, including folders */
	void RepopulateEntireTree();

	/** Adds a new item to the tree if it passes the current filter */
	bool AddItemToTree(WorldHierarchy::FWorldTreeItemRef InItem);

	/** Removes a single item from the tree. This does not delete it */
	void RemoveItemFromTree(WorldHierarchy::FWorldTreeItemRef InItem);

	/** Handles moving an item to ensure that its parent now exists in the hierarchy regardless of filter, or potentially hides the previous parent */
	void OnItemMoved(WorldHierarchy::FWorldTreeItemRef InItem);

	/** Adds a new item to the tree, ignoring the current filter */
	void AddUnfilteredItemToTree(WorldHierarchy::FWorldTreeItemRef InItem);

	/** Called when a child is removed from the parent. This can potentially filter the parent out of the current view */
	void OnChildRemovedFromParent(WorldHierarchy::FWorldTreeItemRef InParent);

	/** Ensures that the item's parent exists and is placed in the hierarchy tree */
	WorldHierarchy::FWorldTreeItemPtr EnsureParentForItem(WorldHierarchy::FWorldTreeItemRef Item);

	/** Gets the expansion state of all parent items. Call this before repopulating the tree */
	TMap<WorldHierarchy::FWorldTreeItemID, bool> GetParentsExpansionState() const;

	/** Sets the expansion state of all parent items based on the cached expansion info */
	void SetParentsExpansionState(const TMap<WorldHierarchy::FWorldTreeItemID, bool>& ExpansionInfo);

	/** Causes the tree to be fully rebuilt */
	void FullRefresh();

	/** Rebuilds the folder list for the hierarchy, and forces a full refresh */
	void RebuildFoldersAndFullRefresh();

	/** Scrolls the specified item into view */
	void ScrollItemIntoView(WorldHierarchy::FWorldTreeItemPtr Item);

	/** Called when a folder is created */
	void OnBroadcastFolderCreate(TSharedPtr<FLevelModel> LevelModel, FName NewPath);

	/** Called when a folder is moved */
	void OnBroadcastFolderMove(TSharedPtr<FLevelModel> LevelModel, FName OldPath, FName NewPath);

	/** Called when a folder is deleted */
	void OnBroadcastFolderDelete(TSharedPtr<FLevelModel> LevelModel, FName Path);

	/** Called when levels have been unloaded */
	void OnBroadcastLevelsUnloaded();

	/** Called when the Create Folder button has been clicked */
	FReply OnCreateFolderClicked();

	/** Requests that items are sorted in the hierarchy view */
	void RequestSort();

	/** Sorts the selected items */
	void SortItems(TArray<WorldHierarchy::FWorldTreeItemPtr>& Items);

	/** Moves the selected items to the given item */
	void MoveSelectionTo(WorldHierarchy::FWorldTreeItemRef Item);

	/** Returns true if the item would pass the current filter */
	bool PassesFilter(const WorldHierarchy::IWorldTreeItem& Item);

	/** Gets the visibility of the empty label */
	EVisibility GetEmptyLabelVisibility() const;

private:
	/** @returns Whether specified item should be expanded */
	bool IsTreeItemExpanded(WorldHierarchy::FWorldTreeItemPtr Item) const;

	/** Appends the Level's name to the OutSearchStrings array if the Level is valid */
	void TransformLevelToString(const FLevelModel* Level, TArray<FString>& OutSearchStrings) const;

	/** Appends the levels contained in Item to the OutSearchStrings array */
	void TransformItemToString(const WorldHierarchy::IWorldTreeItem& Item, TArray<FString>& OutSearchStrings) const;
	
	/** @return Text entered in search box */
	FText GetSearchBoxText() const;

	/** @return	Returns the filter status text */
	FText GetFilterStatusText() const;

	void SetFilterText(const FText& InFilterText);

	/** @return Returns color for the filter status text message, based on success of search filter */
	FSlateColor GetFilterStatusTextColor() const;

	/** @return the content for the view button */
	TSharedRef<SWidget> GetViewButtonContent();

	/** Toggles state of 'display path' */
	void ToggleDisplayPaths_Executed()
	{
		WorldModel->SetDisplayPathsState(!WorldModel->GetDisplayPathsState());
	}

	/** Gets the state of the 'display paths' enabled/disabled */
	bool GetDisplayPathsState() const;

	/** Toggles state of 'display actors count' */
	void ToggleDisplayActorsCount_Executed();

	/** Gets the state of the 'display actors count' enabled/disabled */
	bool GetDisplayActorsCountState() const;

	/** Callback for when the world is saved */
	void OnWorldSaved(UWorld* World, FObjectPostSaveContext ObjectSaveContext);

private:
	/**	Whether the view is currently updating the viewmodel selection */
	bool								bUpdatingSelection;

	/** Our list view widget */
	TSharedPtr<SLevelsTreeWidget>		TreeWidget;
	
	/** Items collection to display */
	TSharedPtr<FLevelCollectionModel>	WorldModel;

	/** The Header Row for the hierarchy */
	TSharedPtr<SHeaderRow>				HeaderRowWidget;

	/**	The LevelTextFilter that constrains which items appear in the world model list */
	TSharedPtr<LevelTextFilter>			SearchBoxLevelFilter;

	/** The filter that constrains which items appear in the hierarchy */
	TSharedPtr<HierarchyFilter>			SearchBoxHierarchyFilter;

	/**	Button representing view options on bottom */
	TSharedPtr<SComboButton>			ViewOptionsComboButton;

	/** Root items for the tree widget */
	TArray<WorldHierarchy::FWorldTreeItemPtr> RootTreeItems;

	/** Reentracy guard */
	bool bIsReentrant;

	/** True if the tree should perform a full refresh */
	bool bFullRefresh;

	/** True if the tree needs to be refreshed */
	bool bNeedsRefresh;

	/** True if the folder list needs to be rebuilt */
	bool bRebuildFolders;

	/** True if the items require sort */
	bool bSortDirty;

	/** Whether the column visibility is currently being changed via code (as opposed to via user selection in the UI). */
	bool bIsProgrammaticallyChangingColumnVisibility = false;

	/** Operations that are waiting to be resolved for items in the tree */
	TArray<WorldHierarchy::FPendingWorldTreeOperation> PendingOperations;

	/** Items that are waiting to be committed to the tree view */
	TMap<WorldHierarchy::FWorldTreeItemID, WorldHierarchy::FWorldTreeItemPtr> PendingTreeItemMap;

	/** All items that are currently displayed in the tree widget */
	TMap<WorldHierarchy::FWorldTreeItemID, WorldHierarchy::FWorldTreeItemPtr> TreeItemMap;

	/** Map of actions to apply to new tree items */
	TMap<WorldHierarchy::FWorldTreeItemID, WorldHierarchy::ENewItemAction> NewItemActions;

	/** The item that is currently pending a rename */
	TWeakPtr<WorldHierarchy::IWorldTreeItem> ItemPendingRename;

	/** Keeps track of which items should be selected after a refresh occurs */
	TArray<WorldHierarchy::FWorldTreeItemID> ItemsSelectedAfterRefresh;

	/** If true, only show root items and the folders for each item */
	bool bFoldersOnlyMode;

	/** If folders-only mode is specified, prevents folders with the following names from being shown in the hierarchy */
	TSet<FName> ExcludedFolders;

	/** Delegate that fires when the selection changes */
	FOnWorldHierarchyItemPicked OnItemPicked;
};