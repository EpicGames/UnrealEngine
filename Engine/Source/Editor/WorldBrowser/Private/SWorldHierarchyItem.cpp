// Copyright Epic Games, Inc. All Rights Reserved.
#include "SWorldHierarchyItem.h"
#include "SlateOptMacros.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Styling/AppStyle.h"
#include "Engine/Engine.h"
#include "ISourceControlProvider.h"
#include "ISourceControlModule.h"
#include "EngineGlobals.h"
#include "Editor.h"
#include "LevelCollectionModel.h"
#include "WorldBrowserDragDrop.h"
#include "WorldHierarchyColumns.h"
#include "SWorldHierarchyImpl.h"

#include "Widgets/Views/SListView.h"
#include "Widgets/Colors/SColorPicker.h"

#include "WorldTreeItemTypes.h"
#include "LevelFolders.h"
#include "WorldBrowserStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "Styling/StyleColors.h"

#define LOCTEXT_NAMESPACE "WorldBrowser"


//////////////////////////
// SWorldHierarchyItem
//////////////////////////

void SWorldHierarchyItem::Construct(const FArguments& InArgs, TSharedRef<STableViewBase> OwnerTableView)
{
	WorldModel = InArgs._InWorldModel;
	WorldTreeItem = InArgs._InItemModel;
	Hierarchy = InArgs._InHierarchy;
	IsItemExpanded = InArgs._IsItemExpanded;
	HighlightText = InArgs._HighlightText;
	bFoldersOnlyMode = InArgs._FoldersOnlyMode;

	FSuperRowType::FArguments Args = FSuperRowType::FArguments();

	if (!bFoldersOnlyMode)
	{
		// Drag should not be detected if the item is only displaying its name
		Args.OnDragDetected(this, &SWorldHierarchyItem::OnItemDragDetected);
	}

	SMultiColumnTableRow<WorldHierarchy::FWorldTreeItemPtr>::Construct(Args, OwnerTableView);
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef< SWidget > SWorldHierarchyItem::GenerateWidgetForColumn( const FName& ColumnID )
{
	using namespace UE::WorldHierarchy;
	TSharedPtr< SWidget > TableRowContent = SNullWidget::NullWidget;

	if (ColumnID == HierarchyColumns::ColumnID_LevelLabel)
	{
		TSharedPtr<SWidget> TextWidget;

		if (!bFoldersOnlyMode && WorldTreeItem->GetAsFolderTreeItem() != nullptr)
		{
			// Folders should support renaming if we're also displaying levels
			TSharedRef<SInlineEditableTextBlock> InlineText = SNew(SInlineEditableTextBlock)
				.Font(this, &SWorldHierarchyItem::GetDisplayNameFont)
				.Text(this, &SWorldHierarchyItem::GetDisplayNameText)
				.ColorAndOpacity(this, &SWorldHierarchyItem::GetDisplayNameColorAndOpacity)
				.HighlightText(HighlightText)
				.ToolTipText(this, &SWorldHierarchyItem::GetDisplayNameTooltip)
				.OnTextCommitted(this, &SWorldHierarchyItem::OnLabelCommitted)
				.OnVerifyTextChanged(this, &SWorldHierarchyItem::OnVerifyItemLabelChanged)
				;

			TextWidget = StaticCastSharedRef<SWidget>(InlineText);

			WorldTreeItem->RenameRequestEvent.BindSP(InlineText, &SInlineEditableTextBlock::EnterEditingMode);
		}
		else
		{
			TextWidget = SNew(STextBlock)
				.Font(this, &SWorldHierarchyItem::GetDisplayNameFont)
				.Text(this, &SWorldHierarchyItem::GetDisplayNameText)
				.ColorAndOpacity(this, &SWorldHierarchyItem::GetDisplayNameColorAndOpacity)
				.HighlightText(HighlightText)
				.ToolTipText(this, &SWorldHierarchyItem::GetDisplayNameTooltip)
				;
		}

		TableRowContent =
			SNew(SHorizontalBox)
			
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SExpanderArrow, SharedThis(this))
			]
			
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SBox)
				.WidthOverride(WorldTreeItem->GetHierarchyItemBrushWidth())
				[
					SNew(SImage)
					.Image(this, &SWorldHierarchyItem::GetLevelIconBrush)
					.ColorAndOpacity(FStyleColors::AccentFolder)
				]
			]

			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(FMargin(5.f, 0.f, 0.f, 0.f))
			[
				TextWidget.ToSharedRef()
			]
		;
	}
	else if (!bFoldersOnlyMode)
	{
		// This is all information we do not want to display or modify if we only want to display item names

		if (ColumnID == HierarchyColumns::ColumnID_LightingScenario)
		{
			TableRowContent =
				SAssignNew(LightingScenarioButton, SButton)
				.ContentPadding(0)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.IsEnabled(this, &SWorldHierarchyItem::IsLightingScenarioEnabled)
				.OnClicked(this, &SWorldHierarchyItem::OnToggleLightingScenario)
				.ToolTipText(this, &SWorldHierarchyItem::GetLightingScenarioToolTip)
				.Visibility(this, &SWorldHierarchyItem::GetLightingScenarioVisibility)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.ForegroundColor_Lambda([this]() { return IsSelected() ? FSlateColor::UseForeground() : FSlateColor::UseStyle(); })
				.Content()
				[
					SNew(SImage)
					.Image(this, &SWorldHierarchyItem::GetLightingScenarioBrush)
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			;
		}
		else if (ColumnID == HierarchyColumns::ColumnID_Lock)
		{
			TableRowContent =
				SAssignNew(LockButton, SButton)
				.ContentPadding(0)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.IsEnabled(this, &SWorldHierarchyItem::IsLockEnabled)
				.OnClicked(this, &SWorldHierarchyItem::OnToggleLock)
				.ToolTipText(this, &SWorldHierarchyItem::GetLevelLockToolTip)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.ForegroundColor_Lambda([this]() { return IsSelected() ? FSlateColor::UseForeground() : FSlateColor::UseStyle(); })
				.Content()
				[
					SNew(SImage)
					.Image(this, &SWorldHierarchyItem::GetLevelLockBrush)
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			;
		}
		else if (ColumnID == HierarchyColumns::ColumnID_EditorVisibility)
		{
			TableRowContent =
				SAssignNew(EditorVisibilityButton, SButton)
				.ContentPadding(0)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.IsEnabled(this, &SWorldHierarchyItem::IsEditorVisibilityEnabled)
				.OnClicked(this, &SWorldHierarchyItem::OnToggleEditorVisibility)
				.ToolTipText(this, &SWorldHierarchyItem::GetEditorVisibilityToolTip)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.ForegroundColor_Lambda([this]() { return IsSelected() ? FSlateColor::UseForeground() : FSlateColor::UseStyle(); })
				.Content()
				[
					SNew(SImage)
					.Image(this, &SWorldHierarchyItem::GetLevelVisibilityBrush)
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			;
		}
		else if (ColumnID == HierarchyColumns::ColumnID_GameVisibility)
		{
			TableRowContent =
			   SAssignNew(GameVisibilityButton, SButton)
			   .ContentPadding(0)
			   .ButtonStyle(FAppStyle::Get(), "SimpleButton")
			   .IsEnabled(this, &SWorldHierarchyItem::IsGameVisibilityEnabled)
			   .OnClicked(this, &SWorldHierarchyItem::OnToggleGameVisibility)
			   .ToolTipText(this, &SWorldHierarchyItem::GetGameVisibilityToolTip)
			   .HAlign(HAlign_Center)
			   .VAlign(VAlign_Center)
			   .ForegroundColor_Lambda([this]() { return IsSelected() ? FSlateColor::UseForeground() : FSlateColor::UseStyle(); })
			   .Content()
			   [
				  SNew(SImage)
				  .Image(this, &SWorldHierarchyItem::GetGameVisibilityBrush)
				  .ColorAndOpacity(FSlateColor::UseForeground())
			   ]
			;
		}
		else if (ColumnID == HierarchyColumns::ColumnID_Color)
		{
			TableRowContent =
				SAssignNew(ColorButton, SButton)
				.ContentPadding(0)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.IsEnabled(true)
				.OnClicked(this, &SWorldHierarchyItem::OnChangeColor)
				.ToolTipText(LOCTEXT("LevelColorButtonToolTip", "Change Level Color"))
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Visibility(this, &SWorldHierarchyItem::GetColorButtonVisibility)
				.Content()
				[
					SNew(SImage)
					.ColorAndOpacity(this, &SWorldHierarchyItem::GetDrawColor)
					.Image(FAppStyle::Get().GetBrush("Level.ColorIcon"))
				]
			;
		}
		else if (ColumnID == HierarchyColumns::ColumnID_Kismet)
		{
			TableRowContent =
				SAssignNew(KismetButton, SButton)
				.ContentPadding(0)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.IsEnabled(this, &SWorldHierarchyItem::IsKismetEnabled)
				.OnClicked(this, &SWorldHierarchyItem::OnOpenKismet)
				.ToolTipText(this, &SWorldHierarchyItem::GetKismetToolTip)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.ForegroundColor_Lambda([this]() { return IsSelected() ? FSlateColor::UseForeground() : FSlateColor::UseStyle(); })
				.Content()
				[
					SNew(SImage)
					.Image(this, &SWorldHierarchyItem::GetLevelKismetBrush)
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			;
		}
		else if (ColumnID == HierarchyColumns::ColumnID_SCCStatus)
		{
			TableRowContent =
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.FillHeight(1.0f)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(this, &SWorldHierarchyItem::GetSCCStateImage)
					.ToolTipText(this, &SWorldHierarchyItem::GetSCCStateTooltip)
				]
			;
		}
		else if (ColumnID == HierarchyColumns::ColumnID_Save)
		{
			TableRowContent =
				SAssignNew(SaveButton, SButton)
				.ContentPadding(0)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.IsEnabled(this, &SWorldHierarchyItem::IsSaveEnabled)
				.OnClicked(this, &SWorldHierarchyItem::OnSave)
				.ToolTipText(this, &SWorldHierarchyItem::GetSaveToolTip)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.ForegroundColor_Lambda([this]() { return IsSelected() ? FSlateColor::UseForeground() : FSlateColor::UseStyle(); })
				.Content()
				[
					SNew(SImage)
					.Image(this, &SWorldHierarchyItem::GetLevelSaveBrush)
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			;
		}
	}

	return TableRowContent.ToSharedRef();
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

FText SWorldHierarchyItem::GetDisplayNameText() const
{
	FFormatNamedArguments Args;
	Args.Add(TEXT("DisplayText"), FText::FromString(WorldTreeItem->GetDisplayString()));
	Args.Add(TEXT("DirtyFlag"), WorldTreeItem->IsDirty() ? FText::FromString(" *") : FText::GetEmpty());
	Args.Add(TEXT("Transient"), (WorldTreeItem->IsTransient() ? LOCTEXT("WorldItem_Transient", " (Transient)") : FText::GetEmpty()));
	Args.Add(TEXT("ReadOnly"), (WorldTreeItem->IsReadOnly() ? LOCTEXT("WorldItem_ReadOnly", " (Read-Only)") : FText::GetEmpty()));

	return FText::Format(LOCTEXT("WorldItem", "{DisplayText}{DirtyFlag}{Transient}{ReadOnly}"), Args);
}

void SWorldHierarchyItem::OnLabelCommitted(const FText& InLabel, ETextCommit::Type InCommitInfo)
{
	WorldHierarchy::FFolderTreeItem* Folder = WorldTreeItem.IsValid() ? WorldTreeItem->GetAsFolderTreeItem() : nullptr;

	if (Folder != nullptr && !InLabel.ToString().Equals(Folder->GetLeafName().ToString(), ESearchCase::CaseSensitive))
	{
		// Rename the item
		FName OldPath = Folder->GetFullPath();
		FName NewPath = WorldHierarchy::GetParentPath(OldPath);
		if (NewPath.IsNone())
		{
			NewPath = FName(*InLabel.ToString());
		}
		else
		{
			NewPath = FName(*(NewPath.ToString() / InLabel.ToString()));
		}

		FLevelFolders::Get().RenameFolder(Folder->GetRootItem().ToSharedRef(), OldPath, NewPath);

		// TODO: Hand focus back to world hierarchy?
	}
}

bool SWorldHierarchyItem::OnVerifyItemLabelChanged(const FText& InLabel, FText& OutErrorMessage)
{
	WorldHierarchy::FFolderTreeItem* Folder = WorldTreeItem.IsValid() ? WorldTreeItem->GetAsFolderTreeItem() : nullptr;

	if (Folder == nullptr)
	{
		OutErrorMessage = LOCTEXT("RenameFailed_TreeItemDeleted", "Folder no longer exists");
		return false;
	}

	FText TrimmedLabel = FText::TrimPrecedingAndTrailing(InLabel);

	if (TrimmedLabel.IsEmpty())
	{
		OutErrorMessage = LOCTEXT("RenameFailed_LeftBlank", "Folder names cannot be left blank");
		return false;
	}

	FString LabelString = TrimmedLabel.ToString();
	if (LabelString.Len() >= NAME_SIZE)
	{
		OutErrorMessage = FText::Format(LOCTEXT("RenameFailed_TooLong", "Names must be less than {0} characters long"), NAME_SIZE);
		return false;
	}

	if (Folder->GetLeafName().ToString() == LabelString)
	{
		return true;
	}

	int32 Dummy = 0;
	if (LabelString.FindChar('/', Dummy) || LabelString.FindChar('\\', Dummy))
	{
		OutErrorMessage = LOCTEXT("RenameFailed_InvalidChar", "Folder names cannot contain / or \\");
		return false;
	}

	// Validate that folder doesn't already exist
	FName NewPath = WorldHierarchy::GetParentPath(Folder->GetFullPath());
	if (NewPath.IsNone())
	{
		NewPath = FName(*LabelString);
	}
	else
	{
		NewPath = FName(*(NewPath.ToString() / LabelString));
	}

	if (FLevelFolders::Get().GetFolderProperties(Folder->GetRootItem().ToSharedRef(), NewPath))
	{
		OutErrorMessage = LOCTEXT("RenameFailed_AlreadyExists", "A folder with this name already exists at this level");
		return false;
	}

	return true;
}

FText SWorldHierarchyItem::GetDisplayNameTooltip() const
{
	return WorldTreeItem->GetToolTipText();
}

bool SWorldHierarchyItem::IsSaveEnabled() const
{
	return WorldTreeItem->CanSave();
}

bool SWorldHierarchyItem::IsLightingScenarioEnabled() const
{
	return WorldTreeItem->HasLightingControls();
}

bool SWorldHierarchyItem::IsLockEnabled() const
{
	return WorldTreeItem->HasLockControls();
}

bool SWorldHierarchyItem::IsEditorVisibilityEnabled() const
{
	return WorldTreeItem->HasEditorVisibilityControls();
}

bool SWorldHierarchyItem::IsGameVisibilityEnabled() const
{
	return WorldTreeItem->HasGameVisibilityControls();
}

bool SWorldHierarchyItem::IsKismetEnabled() const
{
	return WorldTreeItem->HasKismet();
}

FSlateColor SWorldHierarchyItem::GetDrawColor() const
{
	return WorldTreeItem->GetDrawColor();
}

FReply SWorldHierarchyItem::OnToggleEditorVisibility()
{
	if (FSlateApplication::Get().GetModifierKeys().AreModifersDown(EModifierKey::Alt))
	{
		if (WorldTreeItem->IsVisibleInEditor()) 
		{
			WorldTreeItem->OnShowInEditorAllButSelected();
		}
		else
		{
			WorldTreeItem->OnShowInEditorOnlySelected();
		}
	}
	else
	{
		WorldTreeItem->OnToggleEditorVisibility();
	}
	return FReply::Handled();
}

FReply SWorldHierarchyItem::OnToggleGameVisibility()
{
	if (FSlateApplication::Get().GetModifierKeys().AreModifersDown(EModifierKey::Alt))
	{
		if (WorldTreeItem->IsVisibleInEditor()) 
		{
			WorldTreeItem->OnShowInGameAllButSelected();
		}
		else
		{
			WorldTreeItem->OnShowInGameOnlySelected();
		}
	}
	else
	{
		WorldTreeItem->OnToggleGameVisibility();
	}
	return FReply::Handled();
}

FReply SWorldHierarchyItem::OnToggleLightingScenario()
{
	WorldTreeItem->OnToggleLightingScenario();
	return FReply::Handled();
}

FReply SWorldHierarchyItem::OnToggleLock()
{
	if (FSlateApplication::Get().GetModifierKeys().AreModifersDown(EModifierKey::Alt))
	{
		if (WorldTreeItem->IsLocked())
		{
			WorldTreeItem->OnLockAllButSelected();
		}
		else
		{
			WorldTreeItem->OnLockOnlySelected();
		}
	}
	else
	{
		WorldTreeItem->OnToggleLock();
	}
	
	return FReply::Handled();
}

FReply SWorldHierarchyItem::OnSave()
{
	WorldTreeItem->OnSave();
	return FReply::Handled();
}

FReply SWorldHierarchyItem::OnOpenKismet()
{
	WorldTreeItem->OnOpenKismet();
	return FReply::Handled();
}

void SWorldHierarchyItem::OnSetColorFromColorPicker(FLinearColor NewColor)
{
	WorldTreeItem->SetDrawColor(NewColor);
}

void SWorldHierarchyItem::OnColorPickerCancelled(FLinearColor OriginalColor)
{
	WorldTreeItem->SetDrawColor(OriginalColor);
}

void SWorldHierarchyItem::OnColorPickerInteractiveBegin()
{
	GEditor->BeginTransaction(LOCTEXT("EditLevelDragColor", "Edit Level Draw Color"));
}

void SWorldHierarchyItem::OnColorPickerInteractiveEnd()
{
	GEditor->EndTransaction();
}

FReply SWorldHierarchyItem::OnChangeColor()
{
	if (WorldTreeItem->HasColorButtonControls())
	{
		FColorPickerArgs PickerArgs;
		PickerArgs.DisplayGamma = TAttribute<float>::Create(TAttribute<float>::FGetter::CreateUObject(GEngine, &UEngine::GetDisplayGamma));
		PickerArgs.InitialColor = WorldTreeItem->GetDrawColor();
		PickerArgs.bOnlyRefreshOnMouseUp = false;
		PickerArgs.bOnlyRefreshOnOk = false;
		PickerArgs.OnColorCommitted = FOnLinearColorValueChanged::CreateSP(this, &SWorldHierarchyItem::OnSetColorFromColorPicker);
		PickerArgs.OnColorPickerCancelled = FOnColorPickerCancelled::CreateSP(this, &SWorldHierarchyItem::OnColorPickerCancelled);
		PickerArgs.OnInteractivePickBegin = FSimpleDelegate::CreateSP(this, &SWorldHierarchyItem::OnColorPickerInteractiveBegin);
		PickerArgs.OnInteractivePickEnd = FSimpleDelegate::CreateSP(this, &SWorldHierarchyItem::OnColorPickerInteractiveEnd);
		PickerArgs.ParentWidget = AsShared();

		OpenColorPicker(PickerArgs);
	}

	return FReply::Handled();
}

EVisibility SWorldHierarchyItem::GetLightingScenarioVisibility() const
{
	EVisibility Result = EVisibility::Hidden;
	if (WorldTreeItem->HasLightingControls() && WorldTreeItem->GetModel().Num() > 0)
	{
		Result = EVisibility::Visible;
	}
	return Result;
}

EVisibility SWorldHierarchyItem::GetColorButtonVisibility() const
{
	EVisibility Result = EVisibility::Hidden;
	if (WorldTreeItem->HasColorButtonControls())
	{
		Result = EVisibility::Visible;
	}
	return Result;
}

FText SWorldHierarchyItem::GetEditorVisibilityToolTip() const
{
	return WorldTreeItem->GetEditorVisibilityToolTipText();
}

FText SWorldHierarchyItem::GetGameVisibilityToolTip() const
{
	return WorldTreeItem->GetGameVisibilityToolTipText();
}

FText SWorldHierarchyItem::GetSaveToolTip() const
{
	return WorldTreeItem->GetSaveToolTipText();
}

FText SWorldHierarchyItem::GetKismetToolTip() const
{
	FText KismetToolTip;
	if (WorldTreeItem->HasKismet())
	{
		KismetToolTip = LOCTEXT("KismetButtonToolTip", "Open Level Blueprint");
	}
	return KismetToolTip;
}

FReply SWorldHierarchyItem::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& Event)
{
	if (Event.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		FReply Reply = SMultiColumnTableRow<WorldHierarchy::FWorldTreeItemPtr>::OnMouseButtonDown(MyGeometry, Event);

		if (!bFoldersOnlyMode)
		{
			// Drags cannot start if we only want to display item names
			return Reply.DetectDrag( SharedThis(this), EKeys::LeftMouseButton );
		}

		Reply.PreventThrottling();
		return Reply;
	}

	return FReply::Handled();
}

FReply SWorldHierarchyItem::OnItemDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		TSharedPtr<SWorldHierarchyImpl> HierarchyImpl = Hierarchy.Pin();

		if (HierarchyImpl.IsValid())
		{
			TSharedPtr<FDragDropOperation> DragDropOp = WorldHierarchy::CreateDragDropOperation(HierarchyImpl->GetSelectedTreeItems());
			if (DragDropOp.IsValid())
			{
				return FReply::Handled().BeginDragDrop(DragDropOp.ToSharedRef());
			}
		}
	}

	return FReply::Unhandled();
}

FReply SWorldHierarchyItem::OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	WorldHierarchy::FValidationInfo ValidationInfo = WorldTreeItem->ValidateDrop(DragDropEvent);

	if (Hierarchy.Pin().IsValid())
	{
		if (ValidationInfo.bValid)
		{
			WorldTreeItem->OnDrop(DragDropEvent, Hierarchy.Pin().ToSharedRef());
		}
		return FReply::Handled();
	}
	
	return FReply::Unhandled();
}

void SWorldHierarchyItem::OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent )
{
	WorldHierarchy::FValidationInfo ValidationInfo = WorldTreeItem->ValidateDrop(DragDropEvent);

	FName IconName = ValidationInfo.bValid ? TEXT("Graph.ConnectorFeedback.OK") : TEXT("Graph.ConnectorFeedback.Error");
	const FSlateBrush* Icon = FAppStyle::Get().GetBrush(IconName);

	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();
	if (Operation.IsValid())
	{
		if (Operation->IsOfType<WorldHierarchy::FWorldBrowserDragDropOp>())
		{
			TSharedPtr<WorldHierarchy::FWorldBrowserDragDropOp> HierarchyOp = DragDropEvent.GetOperationAs<WorldHierarchy::FWorldBrowserDragDropOp>();
			HierarchyOp->SetToolTip(ValidationInfo.ValidationText, Icon);
		}
		else if (Operation->IsOfType<FAssetDragDropOp>() && !ValidationInfo.ValidationText.IsEmpty())
		{
			TSharedPtr<FAssetDragDropOp> AssetOp = DragDropEvent.GetOperationAs<FAssetDragDropOp>();
			AssetOp->SetToolTip(ValidationInfo.ValidationText, Icon);
		}
	}
}

void SWorldHierarchyItem::OnDragLeave( const FDragDropEvent& DragDropEvent )
{
	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();
	if (Operation.IsValid())
	{
		if (Operation->IsOfType<WorldHierarchy::FWorldBrowserDragDropOp>())
		{
			TSharedPtr<WorldHierarchy::FWorldBrowserDragDropOp> HierarchyOp = DragDropEvent.GetOperationAs<WorldHierarchy::FWorldBrowserDragDropOp>();
			HierarchyOp->ResetToDefaultToolTip();
		}
		else if (Operation->IsOfType<FAssetDragDropOp>())
		{
			TSharedPtr<FAssetDragDropOp> AssetOp = DragDropEvent.GetOperationAs<FAssetDragDropOp>();
			AssetOp->ResetToDefaultToolTip();
		}
	}
}

FSlateFontInfo SWorldHierarchyItem::GetDisplayNameFont() const
{
	if (WorldTreeItem->IsCurrent())
	{
		return FAppStyle::Get().GetFontStyle("WorldBrowser.LabelFontBold");
	}
	else
	{
		return FAppStyle::Get().GetFontStyle("WorldBrowser.LabelFont");
	}
}

FSlateColor SWorldHierarchyItem::GetDisplayNameColorAndOpacity() const
{
	// Force the text to display red if level is missing
	if (!WorldTreeItem->HasValidPackage())
	{
		return FLinearColor(1.0f, 0.0f, 0.0f);
	}
		
	// Highlight text differently if it doesn't match the search filter (e.g., parent levels to child levels that
	// match search criteria.)
	if (WorldTreeItem->Flags.bFilteredOut)
	{
		return FLinearColor(0.30f, 0.30f, 0.30f);
	}

	if (!WorldTreeItem->IsLoaded())
	{
		return FSlateColor::UseSubduedForeground();
	}
		
	if (WorldTreeItem->IsCurrent())
	{
		return WorldTreeItem->GetLevelSelectionFlag() ? FSlateColor::UseForeground() : FLinearColor(0.12f, 0.56f, 1.0f);
	}

	return FSlateColor::UseForeground();
}

const FSlateBrush* SWorldHierarchyItem::GetLevelIconBrush() const
{
	return WorldTreeItem->GetHierarchyItemBrush();
}

const FSlateBrush* SWorldHierarchyItem::GetLevelVisibilityBrush() const
{
	if (WorldTreeItem->HasEditorVisibilityControls())
	{
		if (WorldTreeItem->IsVisibleInEditor())
		{
			return EditorVisibilityButton->IsHovered()
				? FAppStyle::Get().GetBrush( "Level.VisibleHighlightIcon16x" )
				: FAppStyle::Get().GetBrush( "Level.VisibleIcon16x" );
		}
		else
		{
			return EditorVisibilityButton->IsHovered()
				? FAppStyle::Get().GetBrush( "Level.NotVisibleHighlightIcon16x" )
				: FAppStyle::Get().GetBrush( "Level.NotVisibleIcon16x" );
		}
	}
	else
	{
		return FAppStyle::Get().GetBrush( "Level.EmptyIcon16x" );
	}
}

const FSlateBrush* SWorldHierarchyItem::GetGameVisibilityBrush() const
{
	return WorldTreeItem->IsVisibleInGame()
	   ? WorldBrowser::FWorldBrowserStyle::Get().GetBrush( "WorldBrowser.VisibleInGame" )
	   : WorldBrowser::FWorldBrowserStyle::Get().GetBrush( "WorldBrowser.HiddenInGame" );
}

const FSlateBrush* SWorldHierarchyItem::GetLightingScenarioBrush() const
{
	if (WorldTreeItem->IsLightingScenario())
	{
		return FAppStyle::Get().GetBrush( "Level.LightingScenarioIconSolid16x" );
	}
	else
	{
		return FAppStyle::Get().GetBrush( "Level.LightingScenarioNotIconSolid16x" );
	}
}

FText SWorldHierarchyItem::GetLightingScenarioToolTip() const
{
	return LOCTEXT("LightingScenarioButtonToolTip", "Toggle Lighting Scenario");
}

const FSlateBrush* SWorldHierarchyItem::GetLevelLockBrush() const
{
	if (!WorldTreeItem->HasLockControls())
	{
		//Locking the persistent level is not allowed; stub in a different brush
		return FStyleDefaults::GetNoBrush();
	}
	else 
	{
		if (WorldTreeItem->IsLocked())
		{
			return FAppStyle::Get().GetBrush("Icons.Lock");
		}
		else
		{
			return  FAppStyle::Get().GetBrush("Icons.Unlock");
		}
	}
}

FText SWorldHierarchyItem::GetLevelLockToolTip() const
{
	return WorldTreeItem->GetLockToolTipText();
}

FText SWorldHierarchyItem::GetSCCStateTooltip() const
{
	FString PackageName = WorldTreeItem->GetPackageFileName();
	if (!PackageName.IsEmpty())
	{
		FSourceControlStatePtr SourceControlState = ISourceControlModule::Get().GetProvider().GetState(PackageName, EStateCacheUsage::Use);
		if (SourceControlState.IsValid())
		{
			return SourceControlState->GetDisplayTooltip();
		}
	}

	return FText::GetEmpty();
}

const FSlateBrush* SWorldHierarchyItem::GetSCCStateImage() const
{
	FString PackageName = WorldTreeItem->GetPackageFileName();
	if (!PackageName.IsEmpty())
	{
		FSourceControlStatePtr SourceControlState = ISourceControlModule::Get().GetProvider().GetState(PackageName, EStateCacheUsage::Use);
		if(SourceControlState.IsValid())
		{
			return SourceControlState->GetIcon().GetIcon();
		}
	}

	return NULL;
}

const FSlateBrush* SWorldHierarchyItem::GetLevelSaveBrush() const
{
	if (WorldTreeItem->IsDirty())
	{
		return FAppStyle::Get().GetBrush("Icons.SaveModified");
	}
	else
	{
		return FStyleDefaults::GetNoBrush();
	}
}

const FSlateBrush* SWorldHierarchyItem::GetLevelKismetBrush() const
{
	if (WorldTreeItem->IsLoaded())
	{
		if (WorldTreeItem->HasKismet())
		{
			return FAppStyle::Get().GetBrush("Icons.Blueprints");
		}
		else
		{
			return FStyleDefaults::GetNoBrush();
		}
	}
	else
	{
		return FStyleDefaults::GetNoBrush();
	}	
}

#undef LOCTEXT_NAMESPACE
