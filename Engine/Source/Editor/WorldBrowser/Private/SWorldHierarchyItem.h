// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "Styling/SlateColor.h"
#include "Fonts/SlateFontInfo.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"

#include "IWorldTreeItem.h"

class FLevelCollectionModel;
class FLevelModel;
class SButton;
class SWorldHierarchyImpl;

/** A single item in the levels hierarchy tree. Represents a level model */
class SWorldHierarchyItem 
	: public SMultiColumnTableRow<WorldHierarchy::FWorldTreeItemPtr>
{
public:
	DECLARE_DELEGATE_TwoParams( FOnNameChanged, const WorldHierarchy::FWorldTreeItemPtr& /*TreeItem*/, const FVector2D& /*MessageLocation*/);

	SLATE_BEGIN_ARGS( SWorldHierarchyItem )
		: _IsItemExpanded( false )
		, _FoldersOnlyMode( false )
	{}
		/** Data for the world */
		SLATE_ARGUMENT(TSharedPtr<FLevelCollectionModel>, InWorldModel)
	
		/** Item model this widget represents */
		SLATE_ARGUMENT(WorldHierarchy::FWorldTreeItemPtr, InItemModel)

		/** The hierarchy that this item belongs to */
		SLATE_ARGUMENT(TSharedPtr<SWorldHierarchyImpl>, InHierarchy)

		/** True when this item has children and is expanded */
		SLATE_ATTRIBUTE(bool, IsItemExpanded)

		/** The string in the title to highlight */
		SLATE_ATTRIBUTE(FText, HighlightText)

		/** If true, folders cannot be renamed and no other widget information is shown */
		SLATE_ARGUMENT(bool, FoldersOnlyMode)

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, TSharedRef<STableViewBase> OwnerTableView);

	TSharedRef<SWidget> GenerateWidgetForColumn( const FName& ColumnName ) override;

	FReply OnItemDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual void OnDragEnter( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
	virtual FReply OnDragOver( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override { return FReply::Handled(); }
	virtual void OnDragLeave( const FDragDropEvent& DragDropEvent ) override;

protected:
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& Event) override;

private:
	/** Operations buttons enabled/disabled state */
	bool IsSaveEnabled() const;
	bool IsLightingScenarioEnabled() const;
	bool IsLockEnabled() const;
	bool IsEditorVisibilityEnabled() const;
	bool IsGameVisibilityEnabled() const;
	bool IsKismetEnabled() const;

	/** Get DrawColor for the level */
	FSlateColor GetDrawColor() const;
		
	/**
	 *	Called when the user clicks on the visibility icon for a Level's item widget
	 *
	 *	@return	A reply that indicated whether this event was handled.
	 */
	FReply OnToggleEditorVisibility();
	FReply OnToggleGameVisibility();

	FReply OnToggleLightingScenario();

	/**
	 *	Called when the user clicks on the lock icon for a Level's item widget
	 *
	 *	@return	A reply that indicated whether this event was handled.
	 */
	FReply OnToggleLock();

	/**
	 *	Called when the user clicks on the save icon for a Level's item widget
	 *
	 *	@return	A reply that indicated whether this event was handled.
	 */
	FReply OnSave();

	/**
	 *	Called when the user clicks on the kismet icon for a Level's item widget
	 *
	 *	@return	A reply that indicated whether this event was handled.
	 */
	FReply OnOpenKismet();

	/**
	*	Called when the user clicks on the color icon for a Level's item widget
	*
	*	@return	A reply that indicated whether this event was handled.
	*/
	FReply OnChangeColor();

	/** Callback invoked from the color picker */
	void OnSetColorFromColorPicker(FLinearColor NewColor);
	void OnColorPickerCancelled(FLinearColor OriginalColor);
	void OnColorPickerInteractiveBegin();
	void OnColorPickerInteractiveEnd();

	/** Whether the lighting scenario button should be visible */
	EVisibility GetLightingScenarioVisibility() const;

	/** Whether color button should be visible, depends on whether sub-level is loaded */
	EVisibility GetColorButtonVisibility() const;
	
	/** Gets the tooltip for the editor visibility toggle */
	FText GetEditorVisibilityToolTip() const;
	
	/** Gets the tooltip for the game visibility toggle */
	FText GetGameVisibilityToolTip() const;

	/** Gets the tooltip for the save button */
	FText GetSaveToolTip() const;

	/** Gets the tooltip text for the Kismet button */
	FText GetKismetToolTip() const;

	/**
	 *  @return The text of level name while it is not being edited
	 */
	FText GetDisplayNameText() const;

	/**
	*  @return The tooltip of level name (shows the full path of the asset package)
	*/
	FText GetDisplayNameTooltip() const;

	/** */
	FSlateFontInfo GetDisplayNameFont() const;
	
	/** */
	FSlateColor GetDisplayNameColorAndOpacity() const;

	/**
	 *	@return	The SlateBrush representing level icon
	 */
	const FSlateBrush* GetLevelIconBrush() const;

	/**
	 *	Called to get the Slate Image Brush representing the visibility state of
	 *	the Level this item widget represents
	 *
	 *	@return	The SlateBrush representing the Level's visibility state
	 */
	const FSlateBrush* GetLevelVisibilityBrush() const;
	const FSlateBrush* GetGameVisibilityBrush() const;

	const FSlateBrush* GetLightingScenarioBrush() const;

	/**
	 *	Called to get the Slate Image Brush representing the lock state of
	 *	the Level this item widget represents
	 *
	 *	@return	The SlateBrush representing the Level's lock state
	 */
	const FSlateBrush* GetLevelLockBrush() const;

	/**
	 *	Called to get the Slate Image Brush representing the save state of
	 *	the Level this row widget represents
	 *
	 *	@return	The SlateBrush representing the Level's save state
	 */
	const FSlateBrush* GetLevelSaveBrush() const;

	/**
	 *	Called to get the Slate Image Brush representing the kismet state of
	 *	the Level this row widget represents
	 *
	 *	@return	The SlateBrush representing the Level's kismet state
	 */
	const FSlateBrush* GetLevelKismetBrush() const;

	/**
	*	Called to get the Slate Image Brush representing the color of
	*	the Level this row widget represents
	*
	*	@return	The SlateBrush representing the Level's color
	*/
	const FSlateBrush* GetLevelColorBrush() const;

	/** */
	FText GetLightingScenarioToolTip() const;
	
	/** */
	FText GetLevelLockToolTip() const;
	
	/** */
	FText GetSCCStateTooltip() const;
	
	/** */
	const FSlateBrush* GetSCCStateImage() const;

	bool OnVerifyItemLabelChanged(const FText& InLabel, FText& OutErrorMessage);

	void OnLabelCommitted(const FText& InLabel, ETextCommit::Type InCommitInfo);

private:
	/** The world data */
	TSharedPtr<FLevelCollectionModel>	WorldModel;

	/** The data for this item */
	WorldHierarchy::FWorldTreeItemPtr	WorldTreeItem;

	/** The hierarchy for this item */
	TWeakPtr<SWorldHierarchyImpl> Hierarchy;
	
	/** The string to highlight in level display name */
	TAttribute<FText>				HighlightText;

	/** True when this item has children and is expanded */
	TAttribute<bool>				IsItemExpanded;

	/**	The editor visibility button for the Level */
	TSharedPtr<SButton>				EditorVisibilityButton;
	
	/**	The game visibility button for the Level */
	TSharedPtr<SButton>				GameVisibilityButton;

	/**	The lighting scenario button for the Level */
	TSharedPtr<SButton>				LightingScenarioButton;

	/**	The lock button for the Level */
	TSharedPtr<SButton>				LockButton;

	/**	The save button for the Level */
	TSharedPtr<SButton>				SaveButton;

	/**	The kismet button for the Level */
	TSharedPtr<SButton>				KismetButton;

	/**	The color button for the Level */
	TSharedPtr<SButton>				ColorButton;

	/** If true, folders cannot be renamed and only folder names are ever shown */
	bool bFoldersOnlyMode;
};
