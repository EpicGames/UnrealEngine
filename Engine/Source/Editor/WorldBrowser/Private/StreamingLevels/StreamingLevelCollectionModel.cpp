// Copyright Epic Games, Inc. All Rights Reserved.
#include "StreamingLevels/StreamingLevelCollectionModel.h"
#include "Misc/MessageDialog.h"
#include "HAL/FileManager.h"
#include "Modules/ModuleManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Editor/EditorEngine.h"
#include "Settings/LevelEditorMiscSettings.h"
#include "Engine/LevelStreamingAlwaysLoaded.h"
#include "Engine/LevelStreamingDynamic.h"
#include "Editor.h"
#include "EditorDirectories.h"
#include "FileHelpers.h"
#include "Dialogs/Dialogs.h"
#include "ScopedTransaction.h"
#include "EditorLevelUtils.h"
#include "LevelCollectionCommands.h"

#include "Interfaces/IMainFrameModule.h"
#include "DesktopPlatformModule.h"
#include "LevelEditor.h"
#include "LevelEditorActions.h"
#include "NewLevelDialogModule.h"

#include "StreamingLevels/StreamingLevelCustomization.h"
#include "StreamingLevels/StreamingLevelModel.h"
#include "Engine/Selection.h"
#include "Engine/LevelStreamingVolume.h"
#include "GameFramework/WorldSettings.h"

#define LOCTEXT_NAMESPACE "WorldBrowser"

FStreamingLevelCollectionModel::FStreamingLevelCollectionModel()
	: FLevelCollectionModel()
	, AddedLevelStreamingClass(ULevelStreamingDynamic::StaticClass())
	, bAssetDialogOpen(false)
{
	const TSubclassOf<ULevelStreaming> DefaultLevelStreamingClass = GetDefault<ULevelEditorMiscSettings>()->DefaultLevelStreamingClass;
	if ( DefaultLevelStreamingClass )
	{
		AddedLevelStreamingClass = DefaultLevelStreamingClass;
	}
}

FStreamingLevelCollectionModel::~FStreamingLevelCollectionModel()
{
	GEditor->UnregisterForUndo( this );
}

void FStreamingLevelCollectionModel::Initialize(UWorld* InWorld)
{
	BindCommands();	
	GEditor->RegisterForUndo( this );
	
	FLevelCollectionModel::Initialize(InWorld);
}

void FStreamingLevelCollectionModel::OnLevelsCollectionChanged()
{
	InvalidSelectedLevels.Empty();
	
	// We have to have valid world
	if (!CurrentWorld.IsValid())
	{
		return;
	}

	// Add model for a persistent level
	TSharedPtr<FStreamingLevelModel> PersistentLevelModel = MakeShareable(new FStreamingLevelModel(*this, nullptr));
	PersistentLevelModel->SetLevelExpansionFlag(true);
	RootLevelsList.Add(PersistentLevelModel);
	AllLevelsList.Add(PersistentLevelModel);
	AllLevelsMap.Add(PersistentLevelModel->GetLongPackageName(), PersistentLevelModel);
		
	// Add models for each streaming level in the world
	for (ULevelStreaming* StreamingLevel : CurrentWorld->GetStreamingLevels())
	{
		if (StreamingLevel && StreamingLevel->ShowInLevelCollection())
		{
			TSharedPtr<FStreamingLevelModel> LevelModel = MakeShareable(new FStreamingLevelModel(*this, StreamingLevel));
			AllLevelsList.Add(LevelModel);
			AllLevelsMap.Add(LevelModel->GetLongPackageName(), LevelModel);

			PersistentLevelModel->AddChild(LevelModel);
			LevelModel->SetParent(PersistentLevelModel);
		}
	}
	
	FLevelCollectionModel::OnLevelsCollectionChanged();

	// Sync levels selection to world
	SetSelectedLevelsFromWorld();
}

void FStreamingLevelCollectionModel::OnLevelsSelectionChanged()
{
	InvalidSelectedLevels.Empty();

	for (TSharedPtr<FLevelModel> LevelModel : SelectedLevelsList)
	{
		if (!LevelModel->HasValidPackage())
		{
			InvalidSelectedLevels.Add(LevelModel);
		}
	}

	FLevelCollectionModel::OnLevelsSelectionChanged();
}

void FStreamingLevelCollectionModel::UnloadLevels(const FLevelModelList& InLevelList)
{
	if (IsReadOnly())
	{
		return;
	}

	// Persistent level cannot be unloaded
	if (InLevelList.Num() == 1 && InLevelList[0]->IsPersistent())
	{
		return;
	}
		
	bool bHaveDirtyLevels = false;
	for (const TSharedPtr<FLevelModel>& LevelModel : InLevelList)
	{
		if (!bHaveDirtyLevels && LevelModel->IsDirty() && !LevelModel->IsLocked() && !LevelModel->IsPersistent())
		{
			// this level is dirty and can be removed from the world
			bHaveDirtyLevels = true;
		}
	}

	// Depending on the state of the level, create a warning message
	FText LevelWarning = LOCTEXT("RemoveLevel_Undo", "Removing levels cannot be undone.  Proceed?");
	if (bHaveDirtyLevels)
	{
		LevelWarning = LOCTEXT("RemoveLevel_Dirty", "Removing levels cannot be undone.  Any changes to these levels will be lost.  Proceed?");
	}

	// Ask the user if they really wish to remove the level(s)
	FSuppressableWarningDialog::FSetupInfo Info( LevelWarning, LOCTEXT("RemoveLevel_Message", "Remove Level"), "RemoveLevelWarning" );
	Info.ConfirmText = LOCTEXT( "RemoveLevel_Yes", "Yes");
	Info.CancelText = LOCTEXT( "RemoveLevel_No", "No");	
	FSuppressableWarningDialog RemoveLevelWarning( Info );
	if (RemoveLevelWarning.ShowModal() == FSuppressableWarningDialog::Cancel)
	{
		return;
	}
			
	// This will remove streaming levels from a persistent world, so we need to re-populate levels list
	FLevelCollectionModel::UnloadLevels(InLevelList);
	PopulateLevelsList();
}

void FStreamingLevelCollectionModel::AddExistingLevelsFromAssetData(const TArray<FAssetData>& WorldList)
{
	HandleAddExistingLevelSelected(WorldList, false);
}

void FStreamingLevelCollectionModel::BindCommands()
{
	FLevelCollectionModel::BindCommands();
	
	const FLevelCollectionCommands& Commands = FLevelCollectionCommands::Get();
	FUICommandList& ActionList = *CommandList;

	//invalid selected levels
	ActionList.MapAction( Commands.FixUpInvalidReference,
		FExecuteAction::CreateSP( this, &FStreamingLevelCollectionModel::FixupInvalidReference_Executed ) );
	
	ActionList.MapAction( Commands.RemoveInvalidReference,
		FExecuteAction::CreateSP( this, &FStreamingLevelCollectionModel::RemoveInvalidSelectedLevels_Executed ));

	//levels
	ActionList.MapAction( Commands.World_CreateNewLevel,
		FExecuteAction::CreateSP( this, &FStreamingLevelCollectionModel::CreateNewLevel_Executed  ) );
	
	ActionList.MapAction( Commands.World_AddExistingLevel,
		FExecuteAction::CreateSP( this, &FStreamingLevelCollectionModel::AddExistingLevel_Executed ) );

	ActionList.MapAction( Commands.World_AddSelectedActorsToNewLevel,
		FExecuteAction::CreateSP( this, &FStreamingLevelCollectionModel::AddSelectedActorsToNewLevel_Executed  ),
		FCanExecuteAction::CreateSP( this, &FLevelCollectionModel::AreActorsSelected ) );
	
	ActionList.MapAction( Commands.World_RemoveSelectedLevels,
		FExecuteAction::CreateSP( this, &FStreamingLevelCollectionModel::UnloadSelectedLevels_Executed  ),
		FCanExecuteAction::CreateSP( this, &FStreamingLevelCollectionModel::AreAllSelectedLevelsRemovable ) );
	
	ActionList.MapAction( Commands.World_MergeSelectedLevels,
		FExecuteAction::CreateSP( this, &FStreamingLevelCollectionModel::MergeSelectedLevels_Executed  ),
		FCanExecuteAction::CreateSP( this, &FStreamingLevelCollectionModel::AreAllSelectedLevelsEditableAndNotPersistent ) );
	
	// new level streaming method
	ActionList.MapAction( Commands.SetAddStreamingMethod_Blueprint,
		FExecuteAction::CreateSP( this, &FStreamingLevelCollectionModel::SetAddedLevelStreamingClass_Executed, ULevelStreamingDynamic::StaticClass()  ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( this, &FStreamingLevelCollectionModel::IsNewStreamingMethodChecked, ULevelStreamingDynamic::StaticClass()));

	ActionList.MapAction( Commands.SetAddStreamingMethod_AlwaysLoaded,
		FExecuteAction::CreateSP( this, &FStreamingLevelCollectionModel::SetAddedLevelStreamingClass_Executed, ULevelStreamingAlwaysLoaded::StaticClass()  ),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP( this, &FStreamingLevelCollectionModel::IsNewStreamingMethodChecked, ULevelStreamingAlwaysLoaded::StaticClass()));

	// change streaming method
	ActionList.MapAction( Commands.SetStreamingMethod_Blueprint,
		FExecuteAction::CreateSP( this, &FStreamingLevelCollectionModel::SetStreamingLevelsClass_Executed, ULevelStreamingDynamic::StaticClass()  ),
		FCanExecuteAction::CreateSP( this, &FStreamingLevelCollectionModel::AreAllSelectedLevelsEditable ),
		FIsActionChecked::CreateSP( this, &FStreamingLevelCollectionModel::IsStreamingMethodChecked, ULevelStreamingDynamic::StaticClass()));

	ActionList.MapAction( Commands.SetStreamingMethod_AlwaysLoaded,
		FExecuteAction::CreateSP( this, &FStreamingLevelCollectionModel::SetStreamingLevelsClass_Executed, ULevelStreamingAlwaysLoaded::StaticClass()  ),
		FCanExecuteAction::CreateSP( this, &FStreamingLevelCollectionModel::AreAllSelectedLevelsEditable ),
		FIsActionChecked::CreateSP( this, &FStreamingLevelCollectionModel::IsStreamingMethodChecked, ULevelStreamingAlwaysLoaded::StaticClass()));

	ActionList.MapAction( Commands.SetLightingScenario_Enabled,
		FExecuteAction::CreateSP( this, &FStreamingLevelCollectionModel::SetIsLightingScenario, true  ),
		FCanExecuteAction::CreateSP( this, &FStreamingLevelCollectionModel::IsNewLightingScenarioState, true ),
		FIsActionChecked::CreateSP( this, &FStreamingLevelCollectionModel::IsNewLightingScenarioState, false ));

	ActionList.MapAction( Commands.SetLightingScenario_Disabled,
		FExecuteAction::CreateSP( this, &FStreamingLevelCollectionModel::SetIsLightingScenario, false  ),
		FCanExecuteAction::CreateSP( this, &FStreamingLevelCollectionModel::IsNewLightingScenarioState, false ),
		FIsActionChecked::CreateSP( this, &FStreamingLevelCollectionModel::IsNewLightingScenarioState, true ));

	//streaming volume
	ActionList.MapAction( Commands.SelectStreamingVolumes,
		FExecuteAction::CreateSP( this, &FStreamingLevelCollectionModel::SelectStreamingVolumes_Executed  ),
		FCanExecuteAction::CreateSP( this, &FLevelCollectionModel::AreAllSelectedLevelsEditable));

}

TSharedPtr<WorldHierarchy::FWorldBrowserDragDropOp> FStreamingLevelCollectionModel::CreateDragDropOp() const
{
	return CreateDragDropOp(SelectedLevelsList);
}

TSharedPtr<WorldHierarchy::FWorldBrowserDragDropOp> FStreamingLevelCollectionModel::CreateDragDropOp(const FLevelModelList& InLevels) const
{
	TArray<TWeakObjectPtr<ULevelStreaming>> LevelsToDrag;

	for (const TSharedPtr<FLevelModel>& LevelModel : InLevels)
	{
		check(AllLevelsList.Contains(LevelModel));
		TSharedPtr<FStreamingLevelModel> TargetModel = StaticCastSharedPtr<FStreamingLevelModel>(LevelModel);

		if (TargetModel->GetLevelStreaming().IsValid())
		{
			LevelsToDrag.AddUnique(TargetModel->GetLevelStreaming());
		}
	}

	if (LevelsToDrag.Num())
	{
		return WorldHierarchy::FWorldBrowserDragDropOp::New(LevelsToDrag);
	}
	else
	{
		return FLevelCollectionModel::CreateDragDropOp();
	}
}

void FStreamingLevelCollectionModel::BuildHierarchyMenu(FMenuBuilder& InMenuBuilder, EBuildHierarchyMenuFlags Flags) const
{
	const FLevelCollectionCommands& Commands = FLevelCollectionCommands::Get();

	// We show the "level missing" commands, when missing level is selected solely
	if (IsOneLevelSelected() && InvalidSelectedLevels.Num() == 1)
	{
		InMenuBuilder.BeginSection("MissingLevel", LOCTEXT("ViewHeaderRemove", "Missing Level") );
		{
			InMenuBuilder.AddMenuEntry( Commands.FixUpInvalidReference );
			InMenuBuilder.AddMenuEntry( Commands.RemoveInvalidReference );
		}
		InMenuBuilder.EndSection();
	}

	// Add common commands
	InMenuBuilder.BeginSection("Levels", LOCTEXT("LevelsHeader", "Levels") );
	{
		// Make level current
		if (IsOneLevelSelected())
		{
			InMenuBuilder.AddMenuEntry( Commands.World_MakeLevelCurrent );
		}
		
		// Editor Visibility commands
		InMenuBuilder.AddSubMenu( 
			LOCTEXT("EditorVisibilityHeader", "Visibility in Editor"),
			LOCTEXT("EditorVisibilitySubMenu_ToolTip", "Selected Level(s) visibility commands for editor worlds"),
			FNewMenuDelegate::CreateSP(const_cast<FStreamingLevelCollectionModel*>(this), &FStreamingLevelCollectionModel::FillEditorVisibilitySubMenu ) );
		
		// Game Visibility commands
		if (EnumHasAnyFlags(Flags, EBuildHierarchyMenuFlags::ShowGameVisibility))
		{
			InMenuBuilder.AddSubMenu( 
				LOCTEXT("GameVisibilityHeader", "Visibility in Game"),
				LOCTEXT("GameVisibilitySubMenu_ToolTip", "Selected Level(s) visibility commands for game worlds"),
				FNewMenuDelegate::CreateSP(const_cast<FStreamingLevelCollectionModel*>(this), &FStreamingLevelCollectionModel::FillGameVisibilitySubMenu ) );
		}
		

		// Lock commands
		InMenuBuilder.AddSubMenu( 
			LOCTEXT("LockHeader", "Lock"),
			LOCTEXT("LockSubMenu_ToolTip", "Selected Level(s) lock commands"),
			FNewMenuDelegate::CreateSP(const_cast<FStreamingLevelCollectionModel*>(this), &FStreamingLevelCollectionModel::FillLockSubMenu ) );
		
		// Level streaming specific commands
		if (AreAllSelectedLevelsUserManaged() && !(IsOneLevelSelected() && GetSelectedLevels()[0]->IsPersistent()))
		{
			InMenuBuilder.AddMenuEntry(Commands.World_RemoveSelectedLevels);
			//
			InMenuBuilder.AddSubMenu( 
				LOCTEXT("LevelsChangeStreamingMethod", "Change Streaming Method"),
				LOCTEXT("LevelsChangeStreamingMethod_Tooltip", "Changes the streaming method for the selected levels"),
				FNewMenuDelegate::CreateRaw(const_cast<FStreamingLevelCollectionModel*>(this), &FStreamingLevelCollectionModel::FillSetStreamingMethodSubMenu ));
		}

		if (IsOneLevelSelected() && !GetSelectedLevels()[0]->IsPersistent())
		{
			InMenuBuilder.AddSubMenu( 
				LOCTEXT("LevelsChangeLightingScenario", "Lighting Scenario"),
				LOCTEXT("LevelsChangeLightingScenario_Tooltip", "Changes Lighting Scenario Status for the selected level"),
				FNewMenuDelegate::CreateRaw(const_cast<FStreamingLevelCollectionModel*>(this), &FStreamingLevelCollectionModel::FillChangeLightingScenarioSubMenu ));
		}

		InMenuBuilder.AddMenuEntry(Commands.World_FindInContentBrowser);
	}
	InMenuBuilder.EndSection();
	

	// Level selection commands
	InMenuBuilder.BeginSection("LevelsSelection", LOCTEXT("SelectionHeader", "Selection") );
	{
		InMenuBuilder.AddMenuEntry( Commands.SelectAllLevels );
		InMenuBuilder.AddMenuEntry( Commands.DeselectAllLevels );
		InMenuBuilder.AddMenuEntry( Commands.InvertLevelSelection );
	}
	InMenuBuilder.EndSection();
	
	// Level actors selection commands
	InMenuBuilder.BeginSection("Actors", LOCTEXT("ActorsHeader", "Actors") );
	{
		InMenuBuilder.AddMenuEntry( Commands.AddsActors );
		InMenuBuilder.AddMenuEntry( Commands.RemovesActors );

		// Move selected actors to a selected level
		if (IsOneLevelSelected())
		{
			InMenuBuilder.AddMenuEntry( Commands.MoveActorsToSelected );
			InMenuBuilder.AddMenuEntry( Commands.MoveFoliageToSelected );
		}

		if (AreAllSelectedLevelsUserManaged())
		{
			InMenuBuilder.AddMenuEntry(Commands.ConvertLevelToExternalActors);
			InMenuBuilder.AddMenuEntry(Commands.ConvertLevelToInternalActors);
		}

		if (AreAnyLevelsSelected() && !(IsOneLevelSelected() && SelectedLevelsList[0]->IsPersistent()))
		{
			InMenuBuilder.AddMenuEntry( Commands.SelectStreamingVolumes );
		}
	}
	InMenuBuilder.EndSection();
}

void FStreamingLevelCollectionModel::FillSetStreamingMethodSubMenu(FMenuBuilder& InMenuBuilder)
{
	const FLevelCollectionCommands& Commands = FLevelCollectionCommands::Get();
	InMenuBuilder.AddMenuEntry( Commands.SetStreamingMethod_Blueprint, NAME_None, LOCTEXT("SetStreamingMethodBlueprintOverride", "Blueprint") );
	InMenuBuilder.AddMenuEntry( Commands.SetStreamingMethod_AlwaysLoaded, NAME_None, LOCTEXT("SetStreamingMethodAlwaysLoadedOverride", "Always Loaded") );
}

void FStreamingLevelCollectionModel::FillChangeLightingScenarioSubMenu(FMenuBuilder& InMenuBuilder)
{
	const FLevelCollectionCommands& Commands = FLevelCollectionCommands::Get();
	InMenuBuilder.AddMenuEntry( Commands.SetLightingScenario_Enabled, NAME_None, LOCTEXT("SetLightingScenarioEnabled", "Change to Lighting Scenario") );
	InMenuBuilder.AddMenuEntry( Commands.SetLightingScenario_Disabled, NAME_None, LOCTEXT("SetLightingScenarioDisabled", "Change to regular Level") );
}

void FStreamingLevelCollectionModel::CustomizeFileMainMenu(FMenuBuilder& InMenuBuilder) const
{
	FLevelCollectionModel::CustomizeFileMainMenu(InMenuBuilder);

	const FLevelCollectionCommands& Commands = FLevelCollectionCommands::Get();
		
	InMenuBuilder.BeginSection("LevelsAddLevel");
	{
		InMenuBuilder.AddSubMenu( 
			LOCTEXT("LevelsStreamingMethod", "Default Streaming Method"),
			LOCTEXT("LevelsStreamingMethod_Tooltip", "Changes the default streaming method for a new levels"),
			FNewMenuDelegate::CreateRaw(const_cast<FStreamingLevelCollectionModel*>(this), &FStreamingLevelCollectionModel::FillDefaultStreamingMethodSubMenu ) );
		
		InMenuBuilder.AddMenuEntry( Commands.World_CreateNewLevel );
		InMenuBuilder.AddMenuEntry( Commands.World_AddExistingLevel );
		InMenuBuilder.AddMenuEntry( Commands.World_AddSelectedActorsToNewLevel );
		InMenuBuilder.AddMenuEntry( Commands.World_MergeSelectedLevels );
	}
	InMenuBuilder.EndSection();

	InMenuBuilder.BeginSection("Options", LOCTEXT("OptionsLabel", "Options"));
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::Get().LoadModuleChecked<FLevelEditorModule>("LevelEditor");
		TSharedRef<FUICommandList> GlobalLevelCommandList = LevelEditorModule.GetGlobalLevelEditorActions();
		InMenuBuilder.PushCommandList(GlobalLevelCommandList);

		InMenuBuilder.AddMenuEntry(FLevelEditorCommands::Get().OnlyLoadVisibleInPIE);
		InMenuBuilder.AddMenuEntry(FLevelEditorCommands::Get().LevelStreamingVolumePrevis);
	}
	InMenuBuilder.EndSection();
}

void FStreamingLevelCollectionModel::FillDefaultStreamingMethodSubMenu(FMenuBuilder& InMenuBuilder)
{
	const FLevelCollectionCommands& Commands = FLevelCollectionCommands::Get();
	InMenuBuilder.AddMenuEntry( Commands.SetAddStreamingMethod_Blueprint, NAME_None, LOCTEXT("SetAddStreamingMethodBlueprintOverride", "Blueprint") );
	InMenuBuilder.AddMenuEntry( Commands.SetAddStreamingMethod_AlwaysLoaded, NAME_None, LOCTEXT("SetAddStreamingMethodAlwaysLoadedOverride", "Always Loaded") );
}

void FStreamingLevelCollectionModel::RegisterDetailsCustomization(FPropertyEditorModule& InPropertyModule, 
																	TSharedPtr<IDetailsView> InDetailsView)
{
	TSharedRef<FStreamingLevelCollectionModel> WorldModel = StaticCastSharedRef<FStreamingLevelCollectionModel>(this->AsShared());

	InDetailsView->RegisterInstancedCustomPropertyLayout(ULevelStreaming::StaticClass(), 
		FOnGetDetailCustomizationInstance::CreateStatic(&FStreamingLevelCustomization::MakeInstance,  WorldModel)
		);
}

void FStreamingLevelCollectionModel::UnregisterDetailsCustomization(FPropertyEditorModule& InPropertyModule,
																	TSharedPtr<IDetailsView> InDetailsView)
{
	InDetailsView->UnregisterInstancedCustomPropertyLayout(ULevelStreaming::StaticClass());
}

const FLevelModelList& FStreamingLevelCollectionModel::GetInvalidSelectedLevels() const 
{ 
	return InvalidSelectedLevels;
}

//levels
void FStreamingLevelCollectionModel::CreateNewLevel_Executed()
{
	FString TemplateMapPackageName;
	bool bOutIsPartitionedWorld = false;
	const bool bShowPartitionedTemplates = false;
	FNewLevelDialogModule& NewLevelDialogModule = FModuleManager::LoadModuleChecked<FNewLevelDialogModule>("NewLevelDialog");
	IMainFrameModule& MainFrameModule = FModuleManager::LoadModuleChecked<IMainFrameModule>(TEXT("MainFrame"));
	if (NewLevelDialogModule.CreateAndShowNewLevelDialog(MainFrameModule.GetParentWindow(), TemplateMapPackageName, bShowPartitionedTemplates, bOutIsPartitionedWorld))
	{
		UPackage* TemplatePackage = TemplateMapPackageName.Len() ? LoadPackage(nullptr, *TemplateMapPackageName, LOAD_None) : nullptr;
		UWorld* TemplateWorld = TemplatePackage ? UWorld::FindWorldInPackage(TemplatePackage) : nullptr;

		// Create the new level
		EditorLevelUtils::CreateNewStreamingLevelForWorld(*CurrentWorld, AddedLevelStreamingClass, TEXT(""), false, TemplateWorld);

		// Force a cached level list rebuild
		PopulateLevelsList();
	}
}

void FStreamingLevelCollectionModel::AddExistingLevel_Executed()
{
	AddExistingLevel();
}

void FStreamingLevelCollectionModel::AddExistingLevel(bool bRemoveInvalidSelectedLevelsAfter)
{
	if (!bAssetDialogOpen)
	{
		bAssetDialogOpen = true;
		FEditorFileUtils::FOnLevelsChosen LevelsChosenDelegate = FEditorFileUtils::FOnLevelsChosen::CreateSP(this, &FStreamingLevelCollectionModel::HandleAddExistingLevelSelected, bRemoveInvalidSelectedLevelsAfter);
		FEditorFileUtils::FOnLevelPickingCancelled LevelPickingCancelledDelegate = FEditorFileUtils::FOnLevelPickingCancelled::CreateSP(this, &FStreamingLevelCollectionModel::HandleAddExistingLevelCancelled);
		const bool bAllowMultipleSelection = true;
		FEditorFileUtils::OpenLevelPickingDialog(LevelsChosenDelegate, LevelPickingCancelledDelegate, bAllowMultipleSelection);
	}
}

void FStreamingLevelCollectionModel::HandleAddExistingLevelSelected(const TArray<FAssetData>& SelectedAssets, bool bRemoveInvalidSelectedLevelsAfter)
{
	bAssetDialogOpen = false;

	TArray<FString> PackageNames;
	for (const FAssetData& AssetData : SelectedAssets)
	{
		if (ULevel::GetIsLevelPartitionedFromPackage(AssetData.PackageName))
		{
			const FText MessageText = FText::Format(NSLOCTEXT("UnrealEd", "LevelIsPartitioned", "Level ({0}) is partitioned, can't add."), FText::FromName(AssetData.PackageName));

			FSuppressableWarningDialog::FSetupInfo Info(MessageText, LOCTEXT("AddLevelToWorld_Title", "Add Level"), "LevelIsPartitionedWarning");
			Info.ConfirmText = LOCTEXT("IsPartitioned_Ok", "Ok");
			FSuppressableWarningDialog RemoveLevelWarning(Info);
			RemoveLevelWarning.ShowModal();
		}
		else
		{
			PackageNames.Add(AssetData.PackageName.ToString());
		}
	}

	if (PackageNames.Num())
	{
		// Save or selected list, adding a new level will clean it up
		FLevelModelList SavedInvalidSelectedLevels = InvalidSelectedLevels;

		EditorLevelUtils::AddLevelsToWorld(CurrentWorld.Get(), MoveTemp(PackageNames), AddedLevelStreamingClass);

		// Force a cached level list rebuild
		PopulateLevelsList();

		if (bRemoveInvalidSelectedLevelsAfter)
		{
			InvalidSelectedLevels = SavedInvalidSelectedLevels;
			RemoveInvalidSelectedLevels_Executed();
		}
	}
}

void FStreamingLevelCollectionModel::HandleAddExistingLevelCancelled()
{
	bAssetDialogOpen = false;
}

void FStreamingLevelCollectionModel::AddSelectedActorsToNewLevel_Executed()
{
	EditorLevelUtils::CreateNewStreamingLevelForWorld( *CurrentWorld, AddedLevelStreamingClass, TEXT(""), true);

	// Force a cached level list rebuild
	PopulateLevelsList();
}

void FStreamingLevelCollectionModel::FixupInvalidReference_Executed()
{
	// Browsing is essentially the same as adding an existing level
	const bool bRemoveInvalidSelectedLevelsAfter = true;
	AddExistingLevel(bRemoveInvalidSelectedLevelsAfter);
}

void FStreamingLevelCollectionModel::RemoveInvalidSelectedLevels_Executed()
{
	// needs to be an index-based iterator b/c we are removing elements based on it
	for (int32 LevelIdx = InvalidSelectedLevels.Num() - 1; LevelIdx >= 0; LevelIdx--)
	{
		TSharedPtr<FStreamingLevelModel> TargetModel = StaticCastSharedPtr<FStreamingLevelModel>(InvalidSelectedLevels[LevelIdx]);
		ULevelStreaming* LevelStreaming = TargetModel->GetLevelStreaming().Get();

		if (LevelStreaming)
		{
			EditorLevelUtils::RemoveInvalidLevelFromWorld(LevelStreaming);
		}
	}

	// Force a cached level list rebuild
	PopulateLevelsList();
}

void FStreamingLevelCollectionModel::MergeSelectedLevels_Executed()
{
	if (SelectedLevelsList.Num() <= 1)
	{
		return;
	}
	
	// Stash off a copy of the original array, so the selection can be restored
	FLevelModelList SelectedLevelsCopy = SelectedLevelsList;

	//make sure the selected levels are made visible (and thus fully loaded) before merging
	ShowInEditorSelectedLevels_Executed();
	ShowInGameSelectedLevels_Executed();

	//restore the original selection and select all actors in the selected levels
	SetSelectedLevels(SelectedLevelsCopy);
	SelectActors_Executed();

	//Create a new level with the selected actors
	ULevelStreaming* NewStreamingLevel = EditorLevelUtils::CreateNewStreamingLevelForWorld(*CurrentWorld, AddedLevelStreamingClass, TEXT(""), true);

	//If the new level was successfully created (ie the user did not cancel)
	if ((NewStreamingLevel != nullptr) && (CurrentWorld.IsValid()))
	{
		ULevel* NewLevel = NewStreamingLevel->GetLoadedLevel();

		if (CurrentWorld->SetCurrentLevel(NewLevel))
		{
			FEditorDelegates::NewCurrentLevel.Broadcast();
		}

		GEditor->NoteSelectionChange();

		//restore the original selection and remove the levels that were merged
		SetSelectedLevels(SelectedLevelsCopy);
		UnloadSelectedLevels_Executed();
	}

	// Force a cached level list rebuild
	PopulateLevelsList();
}

void FStreamingLevelCollectionModel::SetAddedLevelStreamingClass_Executed(UClass* InClass)
{
	AddedLevelStreamingClass = InClass;
}

bool FStreamingLevelCollectionModel::IsNewStreamingMethodChecked(UClass* InClass) const
{
	return AddedLevelStreamingClass == InClass;
}

bool FStreamingLevelCollectionModel::IsStreamingMethodChecked(UClass* InClass) const
{
	for (const TSharedPtr<FLevelModel>& LevelModel : SelectedLevelsList)
	{
		TSharedPtr<FStreamingLevelModel> TargetModel = StaticCastSharedPtr<FStreamingLevelModel>(LevelModel);
		ULevelStreaming* LevelStreaming = TargetModel->GetLevelStreaming().Get();
		
		if (LevelStreaming && LevelStreaming->GetClass() == InClass)
		{
			return true;
		}
	}
	return false;
}

bool FStreamingLevelCollectionModel::AreAllSelectedLevelsRemovable() const
{
	for (const TSharedPtr<FLevelModel>& LevelModel : SelectedLevelsList)
	{
		if (LevelModel->IsLocked() || LevelModel->IsPersistent() || !LevelModel->IsUserManaged())
		{
			return false;
		}
	}

	return AreAnyLevelsSelected();
}

void FStreamingLevelCollectionModel::SetStreamingLevelsClass_Executed(UClass* InClass)
{
	// First prompt to save the selected levels, as changing the streaming method will unload/reload them
	SaveSelectedLevels_Executed();

	// Stash off a copy of the original array, as changing the streaming method can destroy the selection
	FLevelModelList SelectedLevelsCopy = GetSelectedLevels();

	// Apply the new streaming method to the selected levels
	for (const TSharedPtr<FLevelModel>& LevelModel : SelectedLevelsCopy)
	{
		TSharedPtr<FStreamingLevelModel> TargetModel = StaticCastSharedPtr<FStreamingLevelModel>(LevelModel);
		TargetModel->SetStreamingClass(InClass);
	}

	SetSelectedLevels(SelectedLevelsCopy);

	// Force a cached level list rebuild
	PopulateLevelsList();
}

//streaming volumes
void FStreamingLevelCollectionModel::SelectStreamingVolumes_Executed()
{
	// Iterate over selected levels and make a list of volumes to select.
	TArray<ALevelStreamingVolume*> LevelStreamingVolumesToSelect;
	for (const TSharedPtr<FLevelModel>& LevelModel : SelectedLevelsList)
	{
		TSharedPtr<FStreamingLevelModel> TargetModel = StaticCastSharedPtr<FStreamingLevelModel>(LevelModel);
		ULevelStreaming* StreamingLevel = TargetModel->GetLevelStreaming().Get();

		if (StreamingLevel)
		{
			for (ALevelStreamingVolume* LevelStreamingVolume : StreamingLevel->EditorStreamingVolumes)
			{
				if (LevelStreamingVolume)
				{
					LevelStreamingVolumesToSelect.Add(LevelStreamingVolume);
				}
			}
		}
	}

	// Select the volumes.
	const FScopedTransaction Transaction( LOCTEXT("SelectAssociatedStreamingVolumes", "Select Associated Streaming Volumes") );
	GEditor->GetSelectedActors()->Modify();
	GEditor->SelectNone(false, true);

	for (ALevelStreamingVolume* LevelStreamingVolume : LevelStreamingVolumesToSelect)
	{
		GEditor->SelectActor(LevelStreamingVolume, /*bInSelected=*/ true, false, true);
	}
	
	GEditor->NoteSelectionChange();
}

#undef LOCTEXT_NAMESPACE
