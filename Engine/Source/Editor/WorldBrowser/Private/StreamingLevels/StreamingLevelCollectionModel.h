// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "AssetRegistry/AssetData.h"
#include "EditorUndoClient.h"
#include "LevelCollectionModel.h"
#include "IDetailsView.h"

class FMenuBuilder;

/** The non-UI solution specific presentation logic for a LevelsView */
class FStreamingLevelCollectionModel 
	: public FLevelCollectionModel
	, public FEditorUndoClient
{

public:
	
	/** FStreamingLevelCollectionModel destructor */
	virtual ~FStreamingLevelCollectionModel();

	/**  
	 *	Factory method which creates a new FStreamingLevelCollectionModel object
	 *
	 *	@param	InEditor		The UEditorEngine to use
	 */
	static TSharedRef<FStreamingLevelCollectionModel> Create(UWorld* InWorld)
	{
		TSharedRef<FStreamingLevelCollectionModel> LevelCollectionModel(new FStreamingLevelCollectionModel());
		LevelCollectionModel->Initialize(InWorld);
		return LevelCollectionModel;
	}

public:
	/** FLevelCollection interface */
	virtual void UnloadLevels(const FLevelModelList& InLevelList) override;
	virtual void AddExistingLevelsFromAssetData(const TArray<FAssetData>& WorldList) override;
	virtual TSharedPtr<WorldHierarchy::FWorldBrowserDragDropOp> CreateDragDropOp() const override;
	virtual TSharedPtr<WorldHierarchy::FWorldBrowserDragDropOp> CreateDragDropOp(const FLevelModelList& InLevels) const override;
	virtual void BuildHierarchyMenu(FMenuBuilder& InMenuBuilder, EBuildHierarchyMenuFlags Flags) const override;
	virtual void CustomizeFileMainMenu(FMenuBuilder& InMenuBuilder) const override;
	virtual void RegisterDetailsCustomization(class FPropertyEditorModule& PropertyModule, TSharedPtr<class IDetailsView> InDetailsView)  override;
	virtual void UnregisterDetailsCustomization(class FPropertyEditorModule& PropertyModule, TSharedPtr<class IDetailsView> InDetailsView) override;
	virtual bool HasFolderSupport() const override { return true; }

private:
	virtual void Initialize(UWorld* InWorld) override;
	virtual void BindCommands() override;
	virtual void OnLevelsCollectionChanged() override;
	virtual void OnLevelsSelectionChanged() override;
	/** FLevelCollection interface end */
	
public:
	/** @return Any selected ULevel objects in the LevelsView that are NULL */
	const FLevelModelList& GetInvalidSelectedLevels() const;

private:
	/**  
	 *	FStreamingLevelCollectionModel Constructor
	 *
	 *	@param	InWorldLevels	The Level management logic object
	 *	@param	InEditor		The UEditorEngine to use
	 */
	FStreamingLevelCollectionModel();
	
	//~ Begin FEditorUndoClient Interface
	virtual void PostUndo(bool bSuccess) override { UpdateAllLevels(); }
	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }
	// End of FEditorUndoClient

private:
	/** Adds an existing level; prompts for path */
	void FixupInvalidReference_Executed();

	/** Removes selected levels from world */
	void RemoveInvalidSelectedLevels_Executed();
	
	/** Creates a new level; prompts for level save location */
	void CreateNewLevel_Executed();

	/** Calls AddExistingLevel which adds an existing level; prompts for path */
	void AddExistingLevel_Executed();
	
	/** Adds an existing level; prompts for path, Returns true if a level is selected */
	void AddExistingLevel(bool bRemoveInvalidSelectedLevelsAfter = false);

	/** Handler for when a level is selected after invoking AddExistingLevel */
	void HandleAddExistingLevelSelected(const TArray<FAssetData>& SelectedAssets, bool bRemoveInvalidSelectedLevelsAfter);

	/** Handler for when the level picker dialog is cancelled */
	void HandleAddExistingLevelCancelled();

	/** Add Selected Actors to New Level; prompts for level save location */
	void AddSelectedActorsToNewLevel_Executed();

	/** Merges selected levels into a new level; prompts for level save location */
	void MergeSelectedLevels_Executed();

	/** Changes the streaming method for new or added levels. */
	void SetAddedLevelStreamingClass_Executed(UClass* InClass);

	/** Checks if the passed in streaming method is checked */
	bool IsNewStreamingMethodChecked(UClass* InClass) const;

	/** Checks if the passed in streaming method is the current streaming method */
	bool IsStreamingMethodChecked(UClass* InClass) const;

	/** @return	whether the selected levels are allowed to be removed from the world */
	bool AreAllSelectedLevelsRemovable() const;

	/** Changes the streaming method for the selected levels. */
	void SetStreamingLevelsClass_Executed(UClass* InClass);

	/** Selects the streaming volumes associated with the selected levels */
	void SelectStreamingVolumes_Executed();

	/**  */
	void FillSetStreamingMethodSubMenu(class FMenuBuilder& MenuBuilder);
	
	/**  */
	void FillChangeLightingScenarioSubMenu(class FMenuBuilder& MenuBuilder);

	/**  */
	void FillDefaultStreamingMethodSubMenu(class FMenuBuilder& MenuBuilder);
	
private:
	/** Currently selected NULL Levels */
	FLevelModelList		InvalidSelectedLevels;

	/** The current class to set new or added levels streaming method to. */
	UClass*				AddedLevelStreamingClass;

	/** Boolean indicating whether the asset dialog is currently open */
	bool				bAssetDialogOpen;
};


