// Copyright Epic Games, Inc. All Rights Reserved.
#include "StreamingLevels/StreamingLevelCustomization.h"
#include "Misc/MessageDialog.h"
#include "Widgets/Input/SButton.h"
#include "Engine/LevelStreamingVolume.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "LevelUtils.h"

#include "DetailWidgetRow.h"
#include "PropertyHandle.h"
#include "DetailLayoutBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "DetailCategoryBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Input/SVectorInputBox.h"

#include "StreamingLevels/StreamingLevelModel.h"
#include "StreamingLevels/StreamingLevelEdMode.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Math/UnitConversion.h"
#include "HAL/PlatformApplicationMisc.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "WorldBrowser"

/////////////////////////////////////////////////////
// FWorldTileDetails 
FStreamingLevelCustomization::FStreamingLevelCustomization()
	: bSliderMovement(false)
{
}

TSharedRef<IDetailCustomization> FStreamingLevelCustomization::MakeInstance(TSharedRef<FStreamingLevelCollectionModel> InWorldModel)
{
	TSharedRef<FStreamingLevelCustomization> Instance = MakeShareable(new FStreamingLevelCustomization());
	Instance->WorldModel = InWorldModel;
	return Instance;
}

void FStreamingLevelCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayoutBuilder)
{
	IDetailCategoryBuilder& LevelStreamingCategory = DetailLayoutBuilder.EditCategory("LevelStreaming");

	// Hide level transform
	TSharedPtr<IPropertyHandle> LevelTransformProperty = DetailLayoutBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULevelStreaming, LevelTransform));
	DetailLayoutBuilder.HideProperty(LevelTransformProperty);
	LevelPositionProperty = LevelTransformProperty->GetChildHandle("Translation");
	LevelRotationProperty = LevelTransformProperty->GetChildHandle("Rotation");

	// Add Position property
	LevelStreamingCategory.AddCustomRow(LOCTEXT("Position", "Position"))
		.CopyAction(CreateCopyAction(ELevelTransformField::Location))
		.PasteAction(CreatePasteAction(ELevelTransformField::Location))
		.NameContent()
		[
			SNew(STextBlock)
				.Text(LOCTEXT("Position", "Position"))
				.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
			.MinDesiredWidth(500)
		[
			SNew(SNumericVectorInputBox<FVector::FReal>)
			.IsEnabled(this, &FStreamingLevelCustomization::LevelEditTextTransformAllowed)
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
			.bColorAxisLabels(true)
			.AllowSpin(false)
			.X(this, &FStreamingLevelCustomization::OnGetLevelPosition, 0)
			.Y(this, &FStreamingLevelCustomization::OnGetLevelPosition, 1)
			.Z(this, &FStreamingLevelCustomization::OnGetLevelPosition, 2)
			.OnXCommitted(this, &FStreamingLevelCustomization::OnSetLevelPosition, 0)
			.OnYCommitted(this, &FStreamingLevelCustomization::OnSetLevelPosition, 1)
			.OnZCommitted(this, &FStreamingLevelCustomization::OnSetLevelPosition, 2)
		];

	
	// Add Yaw Rotation property
	LevelStreamingCategory.AddCustomRow(LOCTEXT("Rotation", "Rotation"))
		.CopyAction(CreateCopyAction(ELevelTransformField::Rotation))
		.PasteAction(CreatePasteAction(ELevelTransformField::Rotation))
		.NameContent()
		[
			SNew(STextBlock)
				.Text(LOCTEXT("Rotation", "Rotation"))
				.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
			.MinDesiredWidth(500)
		[
			SNew(SHorizontalBox)

			+SHorizontalBox::Slot()
			.FillWidth(1.f)
			.Padding(0,2,0,2)
			[
				SNew(SNumericEntryBox<int32>)

				.IsEnabled(this, &FStreamingLevelCustomization::LevelEditTextTransformAllowed)
				.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
				.UndeterminedString(LOCTEXT("MultipleValues", "Multiple Values"))
				.AllowSpin(true)
				.MinValue(0)
				.MaxValue(360 - 1)
				.MinSliderValue(0)
				.MaxSliderValue(360 - 1)
				.Value(this, &FStreamingLevelCustomization::GetLevelRotation)
				.OnValueChanged(this, &FStreamingLevelCustomization::OnChangedLevelRotation)
				.OnValueCommitted(this, &FStreamingLevelCustomization::OnSetLevelRotation)
				.OnBeginSliderMovement(this, &FStreamingLevelCustomization::OnBeginLevelRotationSlider)
				.OnEndSliderMovement(this, &FStreamingLevelCustomization::OnEndLevelRotationSlider)
				.LabelPadding(0)					
				.Label()
				[
					SNumericEntryBox<float>::BuildLabel(LOCTEXT("LevelRotation_Label", "Yaw"), FLinearColor::White, SNumericEntryBox<int32>::BlueLabelBackgroundColor)
				]
			]

			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew( SButton )
				.Text(LOCTEXT("EditLevelTransform", "Viewport Edit"))
				.ToolTipText(LOCTEXT("EditLevelToolTip", "Edit level transform in viewport."))
				.OnClicked(this, &FStreamingLevelCustomization::OnEditLevelClicked)
				.IsEnabled(this, &FStreamingLevelCustomization::LevelViewportTransformAllowed)
				.ContentPadding(1)
			]
		];

	TSharedRef<IPropertyHandle> EditorStreamingVolumesProperty = DetailLayoutBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(ULevelStreaming, EditorStreamingVolumes));
	const bool bGenerateHeader = true;
	const bool bDisplayResetToDefault = false;
	TSharedRef<FDetailArrayBuilder> EditorStreamingVolumesBuilder = MakeShareable(new FDetailArrayBuilder(EditorStreamingVolumesProperty, bGenerateHeader, bDisplayResetToDefault));
	EditorStreamingVolumesBuilder->OnGenerateArrayElementWidget(FOnGenerateArrayElementWidget::CreateSP(this, &FStreamingLevelCustomization::OnGenerateElementForEditorStreamingVolume));

	const bool bForAdvanced = false;
	LevelStreamingCategory.AddCustomBuilder(EditorStreamingVolumesBuilder, bForAdvanced);
}

void FStreamingLevelCustomization::OnGenerateElementForEditorStreamingVolume(TSharedRef<IPropertyHandle> ElementProperty, int32 ElementIndex, IDetailChildrenBuilder& ChildrenBuilder)
{
	IDetailPropertyRow& PropertyRow = ChildrenBuilder.AddProperty(ElementProperty);
	TSharedPtr<SWidget> NameWidget;
	TSharedPtr<SWidget> ValueWidget;
	FDetailWidgetRow Row;
	PropertyRow.GetDefaultWidgets(NameWidget, ValueWidget, Row);

	PropertyRow.CustomWidget()
	.NameContent()
	.MinDesiredWidth(Row.NameWidget.MinWidth)
	.MaxDesiredWidth(Row.NameWidget.MaxWidth)
	[
		NameWidget.ToSharedRef()
	]
	.ValueContent()
	.MinDesiredWidth(Row.ValueWidget.MinWidth)
	.MaxDesiredWidth(Row.ValueWidget.MaxWidth)
	[
		SNew(SObjectPropertyEntryBox)
		.PropertyHandle(ElementProperty)
		.AllowedClass(ALevelStreamingVolume::StaticClass())
		.OnShouldSetAsset(this, &FStreamingLevelCustomization::OnShouldSetEditorStreamingVolume, ElementProperty)
	];
}

bool FStreamingLevelCustomization::OnShouldSetEditorStreamingVolume(const FAssetData& AssetData, TSharedRef<IPropertyHandle> ElementProperty) const
{
	ALevelStreamingVolume* Volume = Cast<ALevelStreamingVolume>(AssetData.GetAsset());

	if (Volume != nullptr)
	{
		// Check if there are any duplicates
		bool bIsUnique = true;
		TSharedPtr<IPropertyHandle> ParentProperty = ElementProperty->GetParentHandle();
		TSharedPtr<IPropertyHandleArray> ParentPropertyAsArray = ParentProperty.IsValid() ? ParentProperty->AsArray() : nullptr;
		if (ParentPropertyAsArray.IsValid())
		{
			int32 Index = ElementProperty->GetIndexInArray();
			check(Index != INDEX_NONE);

			uint32 NumItems = 0;
			ensure(ParentPropertyAsArray->GetNumElements(NumItems) == FPropertyAccess::Success);

			for (uint32 ElementIndex = 0; ElementIndex < NumItems; ElementIndex++)
			{
				if (ElementIndex != Index)
				{
					TSharedRef<IPropertyHandle> ElementToCompare = ParentPropertyAsArray->GetElement(ElementIndex);
					UObject* ElementValue = nullptr;
					ensure(ElementToCompare->GetValue(ElementValue) == FPropertyAccess::Success);
					if (ElementValue == Volume)
					{
						FMessageDialog::Open(
							EAppMsgType::Ok,
							LOCTEXT("DuplicateVolume", "This volume is already in the list."));

						bIsUnique = false;
						break;
					}
				}
			}
		}

		// Check that the volume is in the persistent level
		bool bIsInPersistentLevel = Volume->IsInPersistentLevel();

		if (!bIsInPersistentLevel)
		{
			FMessageDialog::Open(
				EAppMsgType::Ok,
				LOCTEXT("VolumeMustBeInPersistentLevel", "Cannot add a Level Streaming Volume which is not in the persistent level."));
		}

		return bIsInPersistentLevel && bIsUnique;
	}
	else if (!AssetData.IsValid())
	{
		// Allow clear
		return true;
	}

	return false;
}

void FStreamingLevelCustomization::OnCopy(ELevelTransformField::Type TransformField)
{

	FString CopyStr;
	switch (TransformField)
	{
	case ELevelTransformField::Location:
		CopyStr = FString::Printf(TEXT("(X=%f,Y=%f,Z=%f)"), OnGetLevelPosition(0).GetValue(), OnGetLevelPosition(1).GetValue(), OnGetLevelPosition(2).GetValue());
		break;
	case ELevelTransformField::Rotation:
		CopyStr = FString::Printf(TEXT("(Pitch=%f,Yaw=%f,Roll=%f)"), 0.0f, (float)GetLevelRotation().GetValue(), 0.0f);
		break;
	default:
		break;
	}

	if (!CopyStr.IsEmpty())
	{
		FPlatformApplicationMisc::ClipboardCopy(*CopyStr);
	}
}

void FStreamingLevelCustomization::OnPaste(ELevelTransformField::Type TransformField)
{
	FString PastedText;
	FPlatformApplicationMisc::ClipboardPaste(PastedText);

	switch (TransformField)
	{
	case ELevelTransformField::Location:
	{
		FVector Location;
		if (Location.InitFromString(PastedText))
		{
			FScopedTransaction Transaction(LOCTEXT("PasteLocation", "Paste Location"));
			OnSetLevelPosition(Location);
		}
	}
	break;
	case ELevelTransformField::Rotation:
	{
		FRotator Rotation;
		PastedText.ReplaceInline(TEXT("Pitch="), TEXT("P="));
		PastedText.ReplaceInline(TEXT("Yaw="), TEXT("Y="));
		PastedText.ReplaceInline(TEXT("Roll="), TEXT("R="));
		if (Rotation.InitFromString(PastedText))
		{
			FScopedTransaction Transaction(LOCTEXT("PasteRotation", "Paste Rotation"));
			OnSetLevelRotation(Rotation.Yaw, ETextCommit::OnEnter);
		}
	}
	break;
	default:
		break;
	}
}

FUIAction FStreamingLevelCustomization::CreateCopyAction(ELevelTransformField::Type TransformField) const
{
	return
		FUIAction
		(
			FExecuteAction::CreateSP(const_cast<FStreamingLevelCustomization*>(this), &FStreamingLevelCustomization::OnCopy, TransformField)
		);
}

FUIAction FStreamingLevelCustomization::CreatePasteAction(ELevelTransformField::Type TransformField) const
{
	return
		FUIAction(FExecuteAction::CreateSP(const_cast<FStreamingLevelCustomization*>(this), &FStreamingLevelCustomization::OnPaste, TransformField));
}

void FStreamingLevelCustomization::OnSetLevelPosition(FVector::FReal NewValue, ETextCommit::Type CommitInfo, int32 Axis )
{
	if (CommitInfo == ETextCommit::Default)
	{
		return;
	}
	TSharedPtr<FStreamingLevelCollectionModel> CollectionModel = WorldModel.Pin();
	if (CollectionModel.IsValid())
	{
		FLevelModelList SelectedLevels = CollectionModel->GetSelectedLevels();
		for (auto It = SelectedLevels.CreateIterator(); It; ++It)
		{
			TSharedPtr<FStreamingLevelModel> LevelModel = StaticCastSharedPtr<FStreamingLevelModel>(*It);
			if (LevelModel->IsEditable() && LevelModel->GetLevelStreaming().IsValid())
			{
				ULevelStreaming* LevelStreaming = LevelModel->GetLevelStreaming().Get();
				// Create transform with new translation
				FTransform LevelTransform = LevelStreaming->LevelTransform;
				FVector LevelTranslation = LevelTransform.GetTranslation();
				LevelTranslation[Axis] = NewValue;
				LevelTransform.SetTranslation(LevelTranslation);
				// Transform level
				FLevelUtils::SetEditorTransform(LevelStreaming, LevelTransform);
			}
		}
	}
}

void FStreamingLevelCustomization::OnSetLevelPosition(FVector NewValue)
{
	TSharedPtr<FStreamingLevelCollectionModel> CollectionModel = WorldModel.Pin();
	if (CollectionModel.IsValid())
	{
		FLevelModelList SelectedLevels = CollectionModel->GetSelectedLevels();
		for (auto It = SelectedLevels.CreateIterator(); It; ++It)
		{
			TSharedPtr<FStreamingLevelModel> LevelModel = StaticCastSharedPtr<FStreamingLevelModel>(*It);
			if (LevelModel->IsEditable() && LevelModel->GetLevelStreaming().IsValid())
			{
				ULevelStreaming* LevelStreaming = LevelModel->GetLevelStreaming().Get();
				// Create transform with new translation
				FTransform LevelTransform = LevelStreaming->LevelTransform;
				LevelTransform.SetTranslation(NewValue);
				// Transform level
				FLevelUtils::SetEditorTransform(LevelStreaming, LevelTransform);
			}
		}
	}
}

TOptional<FVector::FReal> FStreamingLevelCustomization::OnGetLevelPosition( int32 Axis ) const
{
	FVector::FReal AxisVal = 0.f;
	if (LevelPositionProperty->GetChildHandle(Axis)->GetValue(AxisVal) == FPropertyAccess::Success)
	{
		return AxisVal;
	}
	
	return TOptional<FVector::FReal>();
}

void FStreamingLevelCustomization::OnChangedLevelRotation(int32 NewValue)
{
	CachedYawValue = NewValue;
}

void FStreamingLevelCustomization::OnSetLevelRotation( int32 NewValue, ETextCommit::Type CommitInfo)
{
	CachedYawValue = NewValue;
	if (CommitInfo == ETextCommit::Default)
	{
		return;
	}
	
	// Apply rotation when value is committed
	TSharedPtr<FStreamingLevelCollectionModel> CollectionModel = WorldModel.Pin();
	if (CollectionModel.IsValid())
	{
		FQuat NewRotation = FRotator(0.f, (float)CachedYawValue.GetValue(), 0.f).Quaternion();
		
		FLevelModelList SelectedLevels = CollectionModel->GetSelectedLevels();
		for (auto It = SelectedLevels.CreateIterator(); It; ++It)
		{
			TSharedPtr<FStreamingLevelModel> LevelModel = StaticCastSharedPtr<FStreamingLevelModel>(*It);
			if (LevelModel->IsEditable() && LevelModel->GetLevelStreaming().IsValid())
			{
				ULevelStreaming* LevelStreaming = LevelModel->GetLevelStreaming().Get();
				FTransform LevelTransform = LevelStreaming->LevelTransform;
				if (LevelTransform.GetRotation() != NewRotation)
				{
					LevelTransform.SetRotation(NewRotation);
					FLevelUtils::SetEditorTransform(LevelStreaming, LevelTransform);
				}
			}
		}
	}
}

void FStreamingLevelCustomization::OnBeginLevelRotationSlider()
{		
	CachedYawValue = GetLevelRotation();
	bSliderMovement = true;
}

void FStreamingLevelCustomization::OnEndLevelRotationSlider( int32 NewValue )
{
	bSliderMovement = false;
}

TOptional<int32> FStreamingLevelCustomization::GetLevelRotation() const
{
	if (bSliderMovement)
	{
		return CachedYawValue;
	}

	// If were not using the spin box use the actual transform instead of cached as it may have changed with the view port widget
	FQuat RotQ;
	if (LevelRotationProperty->GetValue(RotQ) == FPropertyAccess::Success)
	{
		int32 YawValue = FMath::RoundToInt(RotQ.Rotator().Yaw);
		return YawValue < 0 ? (YawValue + 360) : YawValue;
	}

	return TOptional<int32>();
}

bool FStreamingLevelCustomization::LevelViewportTransformAllowed() const
{
	TSharedPtr<FStreamingLevelCollectionModel> CollectionModel = WorldModel.Pin();
	if (CollectionModel.IsValid() && CollectionModel->IsOneLevelSelected())
	{
		auto SelectedLevel = CollectionModel->GetSelectedLevels()[0];
		return SelectedLevel->IsEditable() && SelectedLevel->IsVisibleInEditor();
	}

	return false;
}

bool FStreamingLevelCustomization::LevelEditTextTransformAllowed() const
{
	TSharedPtr<FStreamingLevelCollectionModel> CollectionModel = WorldModel.Pin();
	if (!CollectionModel.IsValid() || !CollectionModel->AreAnySelectedLevelsEditable())
	{
		return false;
	}
	
	auto LevelModel = StaticCastSharedPtr<FStreamingLevelModel>(CollectionModel->GetSelectedLevels()[0]);
	ULevelStreaming* LevelStreaming = LevelModel->GetLevelStreaming().Get();
	
	auto* ActiveMode = static_cast<FStreamingLevelEdMode*>(
		GLevelEditorModeTools().GetActiveMode(FBuiltinEditorModes::EM_StreamingLevel)
		);		
	
	if (ActiveMode && ActiveMode->IsEditing(LevelStreaming) == true)
	{
		return false;
	}
	
	return CollectionModel->AreAnySelectedLevelsEditable();
}

FReply FStreamingLevelCustomization::OnEditLevelClicked() 
{
	TSharedPtr<FStreamingLevelCollectionModel> CollectionModel = WorldModel.Pin();
	if (!CollectionModel.IsValid() || !CollectionModel->AreAnySelectedLevelsEditable())
	{
		return FReply::Handled();
	}

	auto LevelModel = StaticCastSharedPtr<FStreamingLevelModel>(CollectionModel->GetSelectedLevels()[0]);
	ULevelStreaming* LevelStreaming = LevelModel->GetLevelStreaming().Get();
	
	if (!LevelStreaming)
	{
		return FReply::Handled();
	}		

	if (!GLevelEditorModeTools().IsModeActive(FBuiltinEditorModes::EM_StreamingLevel))
	{
		// Activate Level Mode if it was not active
		GLevelEditorModeTools().ActivateMode(FBuiltinEditorModes::EM_StreamingLevel);
	}
			
	auto* ActiveMode = GLevelEditorModeTools().GetActiveModeTyped<FStreamingLevelEdMode>(FBuiltinEditorModes::EM_StreamingLevel);
	check(ActiveMode != NULL);

	if (ActiveMode->IsEditing(LevelStreaming) == true)
	{
		// Toggle this mode off if already editing this level
		GLevelEditorModeTools().DeactivateMode(FBuiltinEditorModes::EM_StreamingLevel);
		// ActiveMode is now invalid
	}
	else
	{
		// Set the level we now want to edit
		ActiveMode->SetLevel(LevelStreaming);
	}		

	return FReply::Handled();
}



#undef LOCTEXT_NAMESPACE
