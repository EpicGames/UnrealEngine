// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "StreamingLevels/StreamingLevelCollectionModel.h"
#include "IDetailCustomization.h"

class IDetailChildrenBuilder;
class IDetailLayoutBuilder;
class IPropertyHandle;

namespace ELevelTransformField
{
	enum Type
	{
		Location,
		Rotation
	};
}

/////////////////////////////////////////////////////
// FStreamingLevelCustomization
class FStreamingLevelCustomization : public IDetailCustomization
{
public:
	// Makes a new instance of this detail layout class
	static TSharedRef<IDetailCustomization> MakeInstance(TSharedRef<FStreamingLevelCollectionModel> InWorldData);

	// IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
	// End of IDetailCustomization interface

private:
	FStreamingLevelCustomization();

	/**
	 *	Called by the Editor Level Transform column, to set the new values.
	 *
	 *	@param NewValue		The new value entered.
	 *	@param CommitInfo	The way in which the value was committed, currently ignored, we always use the new value.
	 *	@param Axis			The axis being edited.
	 */
	void OnSetLevelPosition( FVector::FReal NewValue, ETextCommit::Type CommitInfo, int32 Axis );

	/**
	 *	Called by the Editor Level Transform column, to set the new values.
	 *
	 *	@param NewValue		The new value entered.
	 */
	void OnSetLevelPosition(FVector NewValue);

	/**
	 *	Called by the Editor Level Transform column, to get the current values.
	 *
	 *	@param Axis		The axis to return the value for.
	 *
	 *	@return			The current value of the translation for given axis.
	 */
	TOptional<FVector::FReal> OnGetLevelPosition( int32 Axis ) const;

	/**
	 *	Called by the Editor Level Rotation column, to update the displayed value as we spin
	 *
	 *	@param NewValue		The new value entered.
	 */
	void OnChangedLevelRotation(int32 NewValue);

	/**
	 *	Called by the Editor Level Rotation column, to set the new values.
	 *
	 *	@param NewValue		The new value entered.
	 *	@param CommitInfo	How the value was set.
	 */
	void OnSetLevelRotation( int32 NewValue, ETextCommit::Type CommitInfo);

	/**
	 *	Called by the Editor Level Rotation column, to set weve started spinning the value.
	 */
	void OnBeginLevelRotationSlider();

	/**
	 *	Called by the Editor Level Rotation column, to set weve stopped spinning the value.
	 *
	 *	@param NewValue		The new value entered.
	 */
	void OnEndLevelRotationSlider( int32 NewValue );

	/**
	 *	Called by the Editor Level Transform column, to get the current values.
	 *
	 *	@return			The current value of the Level Yaw.
	 */
	TOptional<int32> GetLevelRotation() const;

	/** @return Whether we can edit level transform in a viewport */
	bool LevelViewportTransformAllowed() const;

	/** @return Whether we can edit level transform through property editor text fields*/
	bool LevelEditTextTransformAllowed() const;

	/** Handle for edit level transform button */
	FReply OnEditLevelClicked();

	/** Generate custom widget for the EditorStreamingVolumes array customization */
	void OnGenerateElementForEditorStreamingVolume(TSharedRef<IPropertyHandle> ElementProperty, int32 ElementIndex, IDetailChildrenBuilder& ChildrenBuilder);

	/** Called to determine whether the selected streaming volume can be set or not */
	bool OnShouldSetEditorStreamingVolume(const FAssetData& AssetData, TSharedRef<IPropertyHandle> ElementProperty) const;

	/**
	 * Copies the specified transform field to the clipboard
	 */
	void OnCopy(ELevelTransformField::Type TransformField);

	/**
	 * Pastes the specified transform field from the clipboard
	 */
	void OnPaste(ELevelTransformField::Type TransformField);

	/**
	 * Creates a UI action for copying a specified transform field
	 */
	FUIAction CreateCopyAction(ELevelTransformField::Type TransformField) const;

	/**
	 * Creates a UI action for pasting a specified transform field
	 */
	FUIAction CreatePasteAction(ELevelTransformField::Type TransformField) const;

private:
	TWeakPtr<FStreamingLevelCollectionModel>	WorldModel;
	TSharedPtr<class IPropertyHandle>			LevelPositionProperty;
	TSharedPtr<class IPropertyHandle>			LevelRotationProperty;
	bool										bSliderMovement;
	TOptional<int32>							CachedYawValue;
};
