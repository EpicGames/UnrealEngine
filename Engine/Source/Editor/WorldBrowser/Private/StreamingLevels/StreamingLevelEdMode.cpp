// Copyright Epic Games, Inc. All Rights Reserved.

#include "StreamingLevels/StreamingLevelEdMode.h"
#include "Engine/LevelStreaming.h"
#include "EditorViewportClient.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "LevelUtils.h"
#include "ActorEditorUtils.h"
#include "GameFramework/WorldSettings.h"

#define LOCTEXT_NAMESPACE "WorldBrowser"

/** Constructor */
FStreamingLevelEdMode::FStreamingLevelEdMode() 
	: FEdMode()
{
	UMaterial* GizmoMaterial = (UMaterial*)StaticLoadObject( UMaterial::StaticClass(),NULL,TEXT("/Engine/EditorMaterials/LevelTransformMaterial.LevelTransformMaterial") );
	BoxMaterial = UMaterialInstanceDynamic::Create( GizmoMaterial, NULL );

	bIsTracking = false;
}

/** Destructor */
FStreamingLevelEdMode::~FStreamingLevelEdMode()
{
	
}

void FStreamingLevelEdMode::AddReferencedObjects( FReferenceCollector& Collector )
{
	// Call parent implementation
	FEdMode::AddReferencedObjects( Collector );

	Collector.AddReferencedObject(BoxMaterial);
}

EAxisList::Type FStreamingLevelEdMode::GetWidgetAxisToDraw( UE::Widget::EWidgetMode InWidgetMode ) const
{
	switch(InWidgetMode)
	{
	case UE::Widget::WM_Translate:
		return EAxisList::XYZ;
	case UE::Widget::WM_Rotate:
		return EAxisList::Z;
	case UE::Widget::WM_Scale:
		return EAxisList::None;
	default:
		return EAxisList::None;
	}
}

bool FStreamingLevelEdMode::ShouldDrawWidget() const
{
	return SelectedLevel.IsValid();
}

bool FStreamingLevelEdMode::UsesTransformWidget( UE::Widget::EWidgetMode CheckMode ) const 
{
	if( CheckMode == UE::Widget::EWidgetMode::WM_Scale )
	{
		return false;
	}
	return true;
}

FVector FStreamingLevelEdMode::GetWidgetLocation() const
{
	if (SelectedLevel.IsValid())
	{
		return LevelTransform.GetTranslation();
	}
	return FEdMode::GetWidgetLocation();
}

void FStreamingLevelEdMode::SetLevel( ULevelStreaming* LevelStream )
{
	if( SelectedLevel.IsValid() && SelectedLevel.Get() != LevelStream )
	{
		// Changed level need to apply PostEditMove to the actors we moved in the last level
		ApplyPostEditMove();
	}

	SelectedLevel = LevelStream;
	bIsDirty = false;
	if ( SelectedLevel.IsValid() )
	{
		LevelTransform = SelectedLevel->LevelTransform;		
		ULevel* Level = SelectedLevel->GetLoadedLevel();


		// Calculate the Level bounds box
		LevelBounds = FBox(ForceInit);

		if( Level )
		{
			for( int32 ActorIndex=0; ActorIndex< Level->Actors.Num(); ActorIndex++ )
			{
				AActor* Actor = Level->Actors[ActorIndex];

				// Don't include the builder brush or the world settings as they can artificially make the level bounds bigger
				if( Actor && FActorEditorUtils::IsABuilderBrush(Actor) == false && Level->GetWorldSettings() != Actor )
				{
					LevelBounds += Actor->GetComponentsBoundingBox();
				}
			}
		}		
	}	
	GEditor->RedrawAllViewports();
}

bool FStreamingLevelEdMode::InputDelta( FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale )
{
	// Only Update the LevelTransform if the user has clicked on the Widget
	if (InViewportClient->GetCurrentWidgetAxis() != EAxisList::None)
	{
		FVector Translation = LevelTransform.GetTranslation();
		Translation  += InDrag;
		LevelTransform.SetTranslation( Translation );

		FRotator Rot = LevelTransform.GetRotation().Rotator();
		Rot += InRot;
		LevelTransform.SetRotation( Rot.Quaternion() );
		return true;
	}

	return false;
}

void FStreamingLevelEdMode::Render( const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI )
{
	FEdMode::Render( View, Viewport, PDI );

	if( SelectedLevel.IsValid() )
	{	
		if( !bIsTracking && !LevelTransform.Equals( SelectedLevel->LevelTransform ) )
		{
			// Level has changed outside of this tool, probably and undo, need to match up the widget again
			LevelBounds.Min -= ( LevelTransform.GetLocation() - SelectedLevel->LevelTransform.GetLocation() );
			LevelBounds.Max -= ( LevelTransform.GetLocation() - SelectedLevel->LevelTransform.GetLocation() );
			LevelTransform = SelectedLevel->LevelTransform;
		}	

		FTransform BoxTransform = LevelTransform;
		FVector BoxLocation = ( LevelBounds.GetCenter() ) + ( LevelTransform.GetLocation() - SelectedLevel->LevelTransform.GetLocation() );
		BoxTransform.SetLocation( BoxLocation );
		DrawBox( PDI, BoxTransform.ToMatrixWithScale(), LevelBounds.GetExtent(), BoxMaterial->GetRenderProxy(), SDPG_World );
	}
}

bool FStreamingLevelEdMode::StartTracking( FEditorViewportClient* InViewportClient, FViewport* InViewport )
{	
	bIsTracking = true;
	return true;
}

bool FStreamingLevelEdMode::EndTracking( FEditorViewportClient* InViewportClient, FViewport* InViewport )
{	
	bIsTracking = false;
	if( SelectedLevel.IsValid() && !LevelTransform.Equals( SelectedLevel->LevelTransform ) )
	{
		bIsDirty = true;
		// Update Level bounds as its now in a new location
		LevelBounds.Min += ( LevelTransform.GetLocation() - SelectedLevel->LevelTransform.GetLocation() );
		LevelBounds.Max += ( LevelTransform.GetLocation() - SelectedLevel->LevelTransform.GetLocation() );

		// Only update the level transform when the user releases the mouse button
		FLevelUtils::SetEditorTransform( SelectedLevel.Get(), LevelTransform, false );
	}
	return true;
}

void FStreamingLevelEdMode::Enter()
{
	FEdMode::Enter();
	FWorldDelegates::LevelRemovedFromWorld.AddSP(this, &FStreamingLevelEdMode::OnLevelRemovedFromWorld);
}

void FStreamingLevelEdMode::Exit()
{
	FWorldDelegates::LevelRemovedFromWorld.RemoveAll(this);
	ApplyPostEditMove();
	SelectedLevel = NULL;
	FEdMode::Exit();
}

void FStreamingLevelEdMode::ApplyPostEditMove()
{
	if( SelectedLevel.IsValid() && bIsDirty )
	{
		ULevel* LoadedLevel = SelectedLevel->GetLoadedLevel();
		if( LoadedLevel != NULL )
		{			
			FLevelUtils::ApplyPostEditMove(LoadedLevel);
			bIsDirty = false;
		}
	}
}

bool FStreamingLevelEdMode::IsEditing( ULevelStreaming* Level )
{
	return Level == SelectedLevel.Get();
}

bool FStreamingLevelEdMode::IsSnapRotationEnabled()
{
	return true;
}

bool FStreamingLevelEdMode::SnapRotatorToGridOverride( FRotator& Rotation )
{
	 //Rotation = Rotation.GridSnap( FRotator( 90, 90, 90 ) );
	 return true;
}

void FStreamingLevelEdMode::OnLevelRemovedFromWorld(ULevel* InLevel, UWorld* InWorld)
{
	if (SelectedLevel.IsValid())
	{
		if (SelectedLevel->GetLoadedLevel() == InLevel)
		{
			SelectedLevel = NULL;
			Exit();
		}
	}
}

#undef LOCTEXT_NAMESPACE
