// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UnrealWidgetFwd.h"
#include "EdMode.h"

class FEditorViewportClient;
class FPrimitiveDrawInterface;
class FSceneView;
class FViewport;
class ULevelStreaming;
class UMaterialInstanceDynamic;

class FStreamingLevelEdMode : public FEdMode
{
public:

	/** Constructor */
	FStreamingLevelEdMode();

	/** Destructor */
	virtual ~FStreamingLevelEdMode();

	// Begin FEdMode
	virtual void Enter() override;
	virtual void Exit() override; 

	virtual void AddReferencedObjects( FReferenceCollector& Collector ) override;

	virtual EAxisList::Type GetWidgetAxisToDraw( UE::Widget::EWidgetMode InWidgetMode ) const override;

	virtual bool ShouldDrawWidget() const override;

	virtual bool UsesTransformWidget(UE::Widget::EWidgetMode CheckMode) const override;

	virtual FVector GetWidgetLocation() const override;

	virtual bool AllowWidgetMove() override { return true; }

	virtual bool InputDelta( FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale ) override;

	virtual void Render( const FSceneView* View, FViewport* Viewport, FPrimitiveDrawInterface* PDI ) override;

	virtual bool StartTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport) override;

	virtual bool EndTracking(FEditorViewportClient* InViewportClient, FViewport* InViewport) override;

	virtual bool IsSnapRotationEnabled() override;

	virtual bool SnapRotatorToGridOverride(FRotator& Rotation) override;

	// End FEdMode

	/** Sets the level we will be transforming */ 
	void SetLevel( ULevelStreaming* Level );

	/** Returns true if this is the current level were editing  */ 
	virtual bool IsEditing( ULevelStreaming* Level );

	/** Calls Apply Post Edit Move on all Actors in the level*/
	void ApplyPostEditMove();

private:
	/** Called when level was removed from the world */
	void OnLevelRemovedFromWorld(ULevel* InLevel, UWorld* InWorld);

private:
	TWeakObjectPtr<ULevelStreaming> SelectedLevel;
	FTransform LevelTransform;
	TObjectPtr<UMaterialInstanceDynamic> BoxMaterial;
	FBox LevelBounds;
	bool bIsTracking;
	bool bIsDirty;
};

