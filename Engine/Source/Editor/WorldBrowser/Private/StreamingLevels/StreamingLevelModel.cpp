// Copyright Epic Games, Inc. All Rights Reserved.

#include "StreamingLevels/StreamingLevelModel.h"
#include "UObject/UnrealType.h"
#include "Misc/PackageName.h"
#include "Editor.h"
#include "EditorLevelUtils.h"

#include "StreamingLevels/StreamingLevelCollectionModel.h"

#define LOCTEXT_NAMESPACE "WorldBrowser"

FStreamingLevelModel::FStreamingLevelModel(FStreamingLevelCollectionModel& InWorldData, ULevelStreaming* InLevelStreaming)

	: FLevelModel(InWorldData)
	, LevelStreaming(InLevelStreaming)
	, bHasValidPackageName(false)
{
	GEditor->RegisterForUndo( this );
}

FStreamingLevelModel::~FStreamingLevelModel()
{
	GEditor->UnregisterForUndo( this );
}

bool FStreamingLevelModel::HasValidPackage() const
{
	return bHasValidPackageName;
}

UObject* FStreamingLevelModel::GetNodeObject()
{
	return LevelStreaming.Get();
}

ULevel* FStreamingLevelModel::GetLevelObject() const
{
	ULevelStreaming* StreamingObj = LevelStreaming.Get();
	
	if (StreamingObj)
	{
		return StreamingObj->GetLoadedLevel();
	}
	else // persistent level does not have associated level streaming object
	{
		return LevelCollectionModel.GetWorld()->PersistentLevel;
	}
}

bool FStreamingLevelModel::IsTransient() const
{
	if (ULevelStreaming* StreamingObj = LevelStreaming.Get())
	{
		return StreamingObj->HasAnyFlags(RF_Transient);
	}

	return false;
}

FName FStreamingLevelModel::GetAssetName() const
{
	return NAME_None;
}

FName FStreamingLevelModel::GetLongPackageName() const
{
	if (LevelStreaming.IsValid())
	{
		return LevelStreaming->GetWorldAssetPackageFName();
	}
	else
	{
		return LevelCollectionModel.GetWorld()->PersistentLevel->GetOutermost()->GetFName();
	}
}

void FStreamingLevelModel::UpdateAsset(const FAssetData& AssetData)
{
	if (LevelStreaming.IsValid())
	{
		LevelStreaming->SetWorldAssetByPackageName(AssetData.PackageName);
	}
}

FLinearColor FStreamingLevelModel::GetLevelColor() const
{
	if (LevelStreaming.IsValid())
	{
		return LevelStreaming.Get()->LevelColor;
	}
	return FLevelModel::GetLevelColor();
}

void FStreamingLevelModel::SetLevelColor(FLinearColor InColor)
{
	if (LevelStreaming.IsValid())
	{
		FProperty* DrawColorProperty = FindFProperty<FProperty>(LevelStreaming->GetClass(), "LevelColor");
		LevelStreaming->PreEditChange(DrawColorProperty);

		LevelStreaming.Get()->LevelColor = InColor;

		FPropertyChangedEvent PropertyChangedEvent(DrawColorProperty, EPropertyChangeType::ValueSet);
		LevelStreaming->PostEditChangeProperty(PropertyChangedEvent);
	}
}

FName FStreamingLevelModel::GetFolderPath() const
{
	FName FolderPath = NAME_None;
	if (LevelStreaming.IsValid())
	{
		FolderPath = LevelStreaming->GetFolderPath();
	}

	return FolderPath;
}

void FStreamingLevelModel::SetFolderPath(const FName& InFolderPath)
{
	if (LevelStreaming.IsValid())
	{
		LevelStreaming->SetFolderPath(InFolderPath);
	}
}

void FStreamingLevelModel::Update()
{
	UpdatePackageFileAvailability();
	FLevelModel::Update();
}

void FStreamingLevelModel::OnDrop(const TSharedPtr<FLevelDragDropOp>& Op)
{
	TArray<ULevelStreaming*> DropStreamingLevels;
	
	for (TWeakObjectPtr<ULevelStreaming>& WeakLevelStreaming : Op->StreamingLevelsToDrop)
	{
		if (ULevelStreaming* LevelStreamingToDrop = WeakLevelStreaming.Get())
		{
			DropStreamingLevels.AddUnique(LevelStreamingToDrop);
		}
	}	
	
	// Prevent dropping items on itself
	if (DropStreamingLevels.Num())
	{
		ULevelStreaming* StreamingLevel = LevelStreaming.Get();
		if (DropStreamingLevels.Find(StreamingLevel) == INDEX_NONE)
		{
			// Remove streaming level objects from a world streaming levels list
			UWorld* CurrentWorld = LevelCollectionModel.GetWorld();
			CurrentWorld->RemoveStreamingLevels(DropStreamingLevels);

			TArray<ULevelStreaming*> StreamingLevels = CurrentWorld->GetStreamingLevels();

			// Find a new place where to insert the in a world streaming levels list
			// Right after the current level, or at start of the list in case if this is persistent level
			int32 InsertIndex = StreamingLevels.Find(StreamingLevel);
			if (InsertIndex == INDEX_NONE)
			{
				InsertIndex = 0;
			}
			else
			{
				InsertIndex++;
			}

			StreamingLevels.Insert(DropStreamingLevels, InsertIndex);
			CurrentWorld->SetStreamingLevels(MoveTemp(StreamingLevels));

			CurrentWorld->MarkPackageDirty();

			// Force levels list refresh
			LevelCollectionModel.PopulateLevelsList();
		}
	}
}

bool FStreamingLevelModel::IsGoodToDrop(const TSharedPtr<FLevelDragDropOp>& Op) const
{
	return true;
}

//bool FStreamingLevelModel::IsReadOnly() const
//{
//	ULevel* Level = GetLevelObject();
//
//	const UPackage* pPackage = Cast<UPackage>( Level->GetOutermost() );
//	if ( pPackage )
//	{
//		FString PackageFileName;
//		if ( FPackageName::DoesPackageExist( pPackage->GetName(), &PackageFileName ) )
//		{
//			return IFileManager::Get().IsReadOnly( *PackageFileName );
//		}
//	}
//
//	return false;
//}

UClass* FStreamingLevelModel::GetStreamingClass() const
{
	if (!IsPersistent() && LevelStreaming.IsValid())
	{
		return LevelStreaming.Get()->GetClass();
	}
	
	return FLevelModel::GetStreamingClass();
}

void FStreamingLevelModel::SetStreamingClass( UClass *LevelStreamingClass )
{
	if (IsPersistent())
	{
		// Invalid operations for the persistent level
		return;
	}

	ULevelStreaming* StreamingLevel = LevelStreaming.Get();
	if (StreamingLevel)
	{
		StreamingLevel = EditorLevelUtils::SetStreamingClassForLevel( StreamingLevel, LevelStreamingClass );
	}

	Update();
}

const TWeakObjectPtr< ULevelStreaming > FStreamingLevelModel::GetLevelStreaming()
{
	return LevelStreaming;
}

void FStreamingLevelModel::UpdatePackageFileAvailability()
{
	 // Check if streaming level has a valid package name
	if (!GetLevelObject())
	{
		if (LevelStreaming.IsValid())
		{
			FString PackageName = LevelStreaming->PackageNameToLoad == NAME_None ? 
										LevelStreaming->GetWorldAssetPackageName() : 
										LevelStreaming->PackageNameToLoad.ToString();
			
			bHasValidPackageName = FPackageName::DoesPackageExist(PackageName);
		}
		else
		{
			bHasValidPackageName = false;
		}
	}
	else
	{
		bHasValidPackageName = true;
	}
}

#undef LOCTEXT_NAMESPACE
