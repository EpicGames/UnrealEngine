// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"
#include "EditorUndoClient.h"
#include "LevelModel.h"
#include "Layers/Layer.h"

struct FAssetData;
class FLevelDragDropOp;
class FStreamingLevelCollectionModel;
class ULevel;
class ULevelStreaming;

/**
 * The non-UI solution specific presentation logic for a single streaming level
 */
class FStreamingLevelModel 
	: public FLevelModel
	, public FEditorUndoClient	
{

public:
	/**
	 *	FStreamingLevelModel Constructor
	 *
	 *	@param	InEditor			The UEditorEngine to use
	 *	@param	InWorldData			Level collection owning this model
	 *	@param	InLevelStreaming	Streaming object this model should represent
	 */
	FStreamingLevelModel(FStreamingLevelCollectionModel& InWorldData, class ULevelStreaming* InLevelStreaming);
	~FStreamingLevelModel();


public:
	// FLevelModel interface
	virtual bool HasValidPackage() const override;
	virtual UObject* GetNodeObject() override;
	virtual ULevel* GetLevelObject() const override;
	virtual FName GetAssetName() const override;
	virtual FName GetLongPackageName() const override;
	virtual void UpdateAsset(const FAssetData& AssetData) override;
	virtual FLinearColor GetLevelColor() const override;
	virtual void SetLevelColor(FLinearColor InColor) override;
	virtual void Update() override;
	virtual void OnDrop(const TSharedPtr<FLevelDragDropOp>& Op) override;
	virtual bool IsGoodToDrop(const TSharedPtr<FLevelDragDropOp>& Op) const override;
	virtual bool IsTransient() const override;
	virtual UClass* GetStreamingClass() const override;
	FName GetFolderPath() const override;
	virtual void SetFolderPath(const FName& InFolderPath) override;
	virtual bool HasFolderSupport() const override { return true; }
	// FLevelModel interface end
		
	/** @return The ULevelStreaming this viewmodel contains*/
	const TWeakObjectPtr< ULevelStreaming > GetLevelStreaming();

	/** Sets the Level's streaming class */
	void SetStreamingClass( UClass *LevelStreamingClass );
	
private:
	/** Updates cached value of package file availability */
	void UpdatePackageFileAvailability();

	// Begin FEditorUndoClient
	virtual void PostUndo(bool bSuccess) override { Update(); }
	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }
	// End of FEditorUndoClient

private:
	/** The Actor stats of the Level */
	TArray< FLayerActorStats > ActorStats;
	
	/** The LevelStreaming this object represents */
	TWeakObjectPtr< ULevelStreaming > LevelStreaming;

	/** Whether underlying streaming level object has a valid package name */
	bool bHasValidPackageName;
};
