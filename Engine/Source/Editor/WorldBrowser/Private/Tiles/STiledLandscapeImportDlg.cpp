// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tiles/STiledLandscapeImportDlg.h"
#include "Materials/MaterialInterface.h"
#include "AssetRegistry/AssetData.h"
#include "HAL/FileManager.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "EditorDirectories.h"
#include "Widgets/Input/SVectorInputBox.h"
#include "DesktopPlatformModule.h"
#include "LandscapeProxy.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "PropertyCustomizationHelpers.h"
#include "LandscapeFileFormatInterface.h"
#include "LandscapeEditorModule.h"
#include "LandscapeDataAccess.h"
#include "LandscapeUtils.h"

#define LOCTEXT_NAMESPACE "WorldBrowser"

static int32 CalcLandscapeSquareResolution(int32 ComponentsNumX, int32 SectionNumX, int32 SectionQuadsNumX)
{
	return ComponentsNumX * SectionNumX * SectionQuadsNumX + 1;
}

/**
 *	Returns tile coordinates extracted from a specified tile filename
 */
static FIntPoint ExtractTileCoordinates(FString BaseFilename)
{
	//We expect file name in form: <tilename>_x<number>_y<number>
	FIntPoint ResultPosition(-1,-1);
	
	int32 XPos = BaseFilename.Find(TEXT("_x"), ESearchCase::IgnoreCase, ESearchDir::FromEnd);
	int32 YPos = BaseFilename.Find(TEXT("_y"), ESearchCase::IgnoreCase, ESearchDir::FromEnd);
	if (XPos != INDEX_NONE && YPos != INDEX_NONE && XPos < YPos)
	{
		FString XCoord = BaseFilename.Mid(XPos+2, YPos-(XPos+2));
		FString YCoord = BaseFilename.Mid(YPos+2, BaseFilename.Len()-(YPos+2));

		if (XCoord.IsNumeric() && YCoord.IsNumeric())
		{
			TTypeFromString<int32>::FromString(ResultPosition.X, *XCoord);
			TTypeFromString<int32>::FromString(ResultPosition.Y, *YCoord);
		}
	}

	return ResultPosition;
}

void STiledLandscapeImportDlg::Construct(const FArguments& InArgs, TSharedPtr<SWindow> InParentWindow)
{
	bShouldImport = false;
	ParentWindow = InParentWindow;
	
	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SVerticalBox)

			+SVerticalBox::Slot()
			.FillHeight(1)
			.Padding(0,10,0,10)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(2)
				
				// Select tiles
				+SUniformGridPanel::Slot(0, 0)

				+SUniformGridPanel::Slot(1, 0)
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.OnClicked(this, &STiledLandscapeImportDlg::OnClickedSelectHeightmapTiles)
					.Text(LOCTEXT("TiledLandscapeImport_SelectButtonText", "Select Heightmap Tiles..."))
				]

				// Flip Y-Axis orientation
				+SUniformGridPanel::Slot(0, 1)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text( LOCTEXT("TiledLandscapeImport_FlipYAxisText", "Flip Tile Y Coordinate" ) )
				]
				+SUniformGridPanel::Slot(1, 1)
				.VAlign(VAlign_Center)
				[
					SNew(SCheckBox)
					.IsChecked(this, &STiledLandscapeImportDlg::GetFlipYAxisState)
					.OnCheckStateChanged(this, &STiledLandscapeImportDlg::OnFlipYAxisStateChanged)
					.ToolTipText(LOCTEXT("TiledLandscapeImport_FlipYAxisToolTip", "Whether tile Y coordinate should be flipped (Make sure 'Flip Y-Axis Orientation' option is switched off in World Machine) "))
				]

				// Tiles origin offset
				+SUniformGridPanel::Slot(0, 3)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.ToolTipText(LOCTEXT("TiledLandscapeImport_TilesOffsetTooltip", "For example: tile x0_y0 will be treated as x(0+offsetX)_y(0+offsetY)"))
					.Text(LOCTEXT("TiledLandscapeImport_TilesOffsetText", "Tile Coordinates Offset"))
				]

				+SUniformGridPanel::Slot(1, 3)
				.VAlign(VAlign_Center)
				[
					SNew(SHorizontalBox)
					
					+SHorizontalBox::Slot()
					.Padding(0.0f, 1.0f, 2.0f, 1.0f)
					.FillWidth(1.f)
					[
						SNew(SNumericEntryBox<int32>)
						.Value(this, &STiledLandscapeImportDlg::GetTileOffsetX)
						.OnValueChanged(this, &STiledLandscapeImportDlg::SetTileOffsetX)
						.LabelPadding(0)
						.Label()
						[
							SNumericEntryBox<int32>::BuildLabel( LOCTEXT("X_Label", "X"), FLinearColor::White, SNumericEntryBox<int32>::RedLabelBackgroundColor )
						]
					]
					
					+SHorizontalBox::Slot()
					.Padding(0.0f, 1.0f, 2.0f, 1.0f)
					.FillWidth(1.f)
					[
						SNew(SNumericEntryBox<int32>)
						.Value(this, &STiledLandscapeImportDlg::GetTileOffsetY)
						.OnValueChanged(this, &STiledLandscapeImportDlg::SetTileOffsetY)
						.LabelPadding(0)
						.Label()
						[
							SNumericEntryBox<int32>::BuildLabel( LOCTEXT("Y_Label", "Y"), FLinearColor::White, SNumericEntryBox<int32>::GreenLabelBackgroundColor )
						]
					]
				]
				
				// Tile configuration
				+SUniformGridPanel::Slot(0, 4)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("TiledLandscapeImport_ConfigurationText", "Import Configuration"))
				]

				+SUniformGridPanel::Slot(1, 4)
				.VAlign(VAlign_Center)
				[
					SAssignNew(TileConfigurationComboBox, SComboBox<TSharedPtr<FTileImportConfiguration>>)
					.OptionsSource(&ActiveConfigurations)
					.OnSelectionChanged(this, &STiledLandscapeImportDlg::OnSetImportConfiguration)
					.OnGenerateWidget(this, &STiledLandscapeImportDlg::HandleTileConfigurationComboBoxGenarateWidget)
					.Content()
					[
						SNew(STextBlock)
						.Text(this, &STiledLandscapeImportDlg::GetTileConfigurationText)
					]
				]

				// Scale
				+SUniformGridPanel::Slot(0, 5)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("TiledLandscapeImport_ScaleText", "Landscape Scale"))
				]
			
				+SUniformGridPanel::Slot(1, 5)
				.VAlign(VAlign_Center)
				[
					SNew( SVectorInputBox )
					.bColorAxisLabels( true )
					.AllowSpin(false)
					.X( this, &STiledLandscapeImportDlg::GetScaleX )
					.Y( this, &STiledLandscapeImportDlg::GetScaleY )
					.Z( this, &STiledLandscapeImportDlg::GetScaleZ )
					.OnXCommitted( this, &STiledLandscapeImportDlg::OnSetScale, 0 )
					.OnYCommitted( this, &STiledLandscapeImportDlg::OnSetScale, 1 )
					.OnZCommitted( this, &STiledLandscapeImportDlg::OnSetScale, 2 )
				]

				// Landcape material
				+SUniformGridPanel::Slot(0, 6)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("TiledLandscapeImport_MaterialText", "Material"))
				]
			
				+SUniformGridPanel::Slot(1, 6)
				.VAlign(VAlign_Center)
				[
					SNew(SObjectPropertyEntryBox)
					.AllowedClass(UMaterialInterface::StaticClass())
					.ObjectPath(this, &STiledLandscapeImportDlg::GetLandscapeMaterialPath)
					.OnObjectChanged(this, &STiledLandscapeImportDlg::OnLandscapeMaterialChanged)
					.AllowClear(true)
				]
			]

			// Layers
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
			[
				SAssignNew(LayerDataListView, SListView<TSharedPtr<FTiledLandscapeImportSettings::LandscapeLayerSettings>>)
				.ListItemsSource( &LayerDataList )
				.OnGenerateRow( this, &STiledLandscapeImportDlg::OnGenerateWidgetForLayerDataListView )
				.SelectionMode(ESelectionMode::None)
			]

			// Import summary
			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
			[
				SNew(STextBlock)
				.Text(this, &STiledLandscapeImportDlg::GetImportSummaryText)
				.WrapTextAt(600.0f)
			]

			// Import, Cancel
			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Bottom)
			.Padding(0,10,0,10)
			[
				SNew(SUniformGridPanel)
				.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
				.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
				.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
				+SUniformGridPanel::Slot(0,0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.IsEnabled(this, &STiledLandscapeImportDlg::IsImportEnabled)
					.OnClicked(this, &STiledLandscapeImportDlg::OnClickedImport)
					.Text(LOCTEXT("TiledLandscapeImport_ImportButtonText", "Import"))
				]
				+SUniformGridPanel::Slot(1,0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.OnClicked(this, &STiledLandscapeImportDlg::OnClickedCancel)
					.Text(LOCTEXT("TiledLandscapeImport_CancelButtonText", "Cancel"))
				]
			]
		]
	];

	GenerateAllPossibleTileConfigurations();
	ImportSettings.ComponentsNum = 0;
}

TSharedRef<SWidget> STiledLandscapeImportDlg::HandleTileConfigurationComboBoxGenarateWidget(TSharedPtr<FTileImportConfiguration> InItem) const
{
	const FText ItemText = GenerateConfigurationText(InItem->NumComponents, InItem->NumSectionsPerComponent, InItem->NumQuadsPerSection);
		
	return SNew(SBox)
	.Padding(4)
	[
		SNew(STextBlock).Text(ItemText)
	];
}

FText STiledLandscapeImportDlg::GetTileConfigurationText() const
{
	if (ImportSettings.HeightmapFileList.Num() == 0)
	{
		return LOCTEXT("TiledLandscapeImport_NoTilesText", "No tiles selected");
	}

	if (ImportSettings.SectionsPerComponent <= 0)
	{
		return LOCTEXT("TiledLandscapeImport_InvalidTileResolutionText", "Selected tiles have unsupported resolution");
	}
	
	return GenerateConfigurationText(ImportSettings.ComponentsNum, ImportSettings.SectionsPerComponent, ImportSettings.QuadsPerSection);
}

TSharedRef<ITableRow> STiledLandscapeImportDlg::OnGenerateWidgetForLayerDataListView(
	TSharedPtr<FTiledLandscapeImportSettings::LandscapeLayerSettings> InLayerData, 
	const TSharedRef<STableViewBase>& OwnerTable)
{
	return 	SNew(STableRow<TSharedRef<FTiledLandscapeImportSettings::LandscapeLayerSettings>>, OwnerTable)
			[
				SNew(SBorder)
				[
					SNew(SHorizontalBox)

					// Layer name
					+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Left)
					.FillWidth(1)
					[
						SNew(STextBlock).Text(FText::FromName(InLayerData->Name))
					]

					// Blend option
					+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Center)
					.Padding(2)
					.AutoWidth()
					[
						SNew(SCheckBox)
						.IsChecked(this, &STiledLandscapeImportDlg::GetLayerBlendState, InLayerData)
						.OnCheckStateChanged(this, &STiledLandscapeImportDlg::OnLayerBlendStateChanged, InLayerData)
						.ToolTipText(LOCTEXT("TiledLandscapeImport_BlendOption", "Weight-Blended Layer"))
					]
					
					// Number of selected weightmaps
					+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Right)
					.Padding(2)
					.AutoWidth()
					[
						SNew(STextBlock).Text(this, &STiledLandscapeImportDlg::GetWeightmapCountText, InLayerData)
					]
					
					// Button for selecting weightmap files
					+SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Right)
					.AutoWidth()
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
						.OnClicked(this, &STiledLandscapeImportDlg::OnClickedSelectWeightmapTiles, InLayerData)
						.Text(LOCTEXT("TiledLandscapeImport_SelectWeightmapButtonText", "Select Weightmap Tiles..."))
					]
				]
			];
	
}

bool STiledLandscapeImportDlg::ShouldImport() const
{
	return bShouldImport;
}

const FTiledLandscapeImportSettings& STiledLandscapeImportDlg::GetImportSettings() const
{
	return ImportSettings;
}

TOptional<float> STiledLandscapeImportDlg::GetScaleX() const
{
	return ImportSettings.Scale3D.X;
}

TOptional<float> STiledLandscapeImportDlg::GetScaleY() const
{
	return ImportSettings.Scale3D.Y;
}

TOptional<float> STiledLandscapeImportDlg::GetScaleZ() const
{
	return ImportSettings.Scale3D.Z;
}

void STiledLandscapeImportDlg::OnSetScale(float InValue, ETextCommit::Type CommitType, int32 InAxis)
{
	if (InAxis < 2) //XY uniform
	{
		ImportSettings.Scale3D.X = FMath::Abs(InValue);
		ImportSettings.Scale3D.Y = FMath::Abs(InValue);
	}
	else //Z
	{
		ImportSettings.Scale3D.Z = FMath::Abs(InValue);
	}
}

TOptional<int32> STiledLandscapeImportDlg::GetTileOffsetX() const
{
	return ImportSettings.TilesCoordinatesOffset.X;
}

void STiledLandscapeImportDlg::SetTileOffsetX(int32 InValue)
{
	ImportSettings.TilesCoordinatesOffset.X = InValue;
}

TOptional<int32> STiledLandscapeImportDlg::GetTileOffsetY() const
{
	return ImportSettings.TilesCoordinatesOffset.Y;
}

void STiledLandscapeImportDlg::SetTileOffsetY(int32 InValue)
{
	ImportSettings.TilesCoordinatesOffset.Y = InValue;
}

ECheckBoxState STiledLandscapeImportDlg::GetFlipYAxisState() const
{
	return ImportSettings.bFlipYAxis ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void STiledLandscapeImportDlg::OnFlipYAxisStateChanged(ECheckBoxState NewState)
{
	ImportSettings.bFlipYAxis = (NewState == ECheckBoxState::Checked);
}

void STiledLandscapeImportDlg::OnSetImportConfiguration(TSharedPtr<FTileImportConfiguration> InTileConfig, ESelectInfo::Type SelectInfo)
{
	if (InTileConfig.IsValid())
	{
		ImportSettings.ComponentsNum = InTileConfig->NumComponents;
		ImportSettings.QuadsPerSection = InTileConfig->NumQuadsPerSection;
		ImportSettings.SectionsPerComponent = InTileConfig->NumSectionsPerComponent;
		ImportSettings.SizeX = InTileConfig->SizeX;
	}
	else
	{
		ImportSettings.ComponentsNum = 0;
		ImportSettings.HeightmapFileList.Empty();
	}
}

FReply STiledLandscapeImportDlg::OnClickedSelectHeightmapTiles()
{
	TotalLandscapeRect = FIntRect(MAX_int32, MAX_int32, MIN_int32, MIN_int32);
	ImportSettings.HeightmapFileList.Empty();
	ImportSettings.TileCoordinates.Empty();

	ActiveConfigurations.Empty();
	ImportSettings.ComponentsNum = 0;
	StatusMessage = FText();

	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (DesktopPlatform)
	{
		if (ParentWindow->GetNativeWindow().IsValid())
		{
			void* ParentWindowWindowHandle = ParentWindow->GetNativeWindow()->GetOSWindowHandle();

			ILandscapeEditorModule& LandscapeEditorModule = FModuleManager::GetModuleChecked<ILandscapeEditorModule>("LandscapeEditor");
			const TCHAR* FileTypes = LandscapeEditorModule.GetHeightmapImportDialogTypeString();

			bool bOpened = DesktopPlatform->OpenFileDialog(
								ParentWindowWindowHandle,
								LOCTEXT("SelectHeightmapTiles", "Select heightmap tiles").ToString(),
								*FEditorDirectories::Get().GetLastDirectory(ELastDirectory::UNR),
								TEXT(""),
								FileTypes,
								EFileDialogFlags::Multiple,
								ImportSettings.HeightmapFileList);

			if (bOpened && ImportSettings.HeightmapFileList.Num() > 0)
			{
				IFileManager& FileManager = IFileManager::Get();
				bool bValidTiles = true;
				int32 TargetSizeX = 0;

				const FString TargetExtension = FPaths::GetExtension(ImportSettings.HeightmapFileList[0], true);
				const ILandscapeHeightmapFileFormat* HeightmapFormat = LandscapeEditorModule.GetHeightmapFormatByExtension(*TargetExtension);

				// All heightmap tiles have to be the same size and have correct tile position encoded into filename
				for (const FString& Filename : ImportSettings.HeightmapFileList)
				{
					FFormatNamedArguments Arguments;

					FString BaseFilename = FPaths::GetBaseFilename(Filename);
					FString CleanFilename = FPaths::GetCleanFilename(Filename);

					Arguments.Add(TEXT("FileName"), FText::FromString(CleanFilename));

					FText FilenameError;
					if (FPackageName::DoesPackageNameContainInvalidCharacters(BaseFilename, &FilenameError))
					{
						Arguments.Add(TEXT("FilenameError"), FilenameError);
						StatusMessage = FText::Format(LOCTEXT("TiledLandscapeImport_HeightmapTileInvalidName", "File name ({FileName}) contains invalid characters. {FilenameError}"), Arguments);
						bValidTiles = false;
						break;
					}

					FIntPoint TileCoordinate = ExtractTileCoordinates(BaseFilename);
					if (TileCoordinate.GetMin() < 0)
					{
						StatusMessage = FText::Format(LOCTEXT("TiledLandscapeImport_HeightmapTileInvalidPattern", "File name ({FileName}) should match pattern: <name>_X<number>_Y<number>.<extension>"), Arguments);
						bValidTiles = false;
						break;
					}

					if (!Filename.EndsWith(TargetExtension))
					{
						StatusMessage = FText::Format(LOCTEXT("TiledLandscapeImport_HeightmapMixedFileTypes", "File ({FileName}) has a different file extension, please use all the same type (16-bit grayscale png preferred)."), Arguments);
						bValidTiles = false;
						break;
					}

					if (!HeightmapFormat)
					{
						StatusMessage = FText::Format(LOCTEXT("TiledLandscapeImport_UnrecognisedExtension", "Error loading file ({FileName}), unrecognised extension."), Arguments);
						bValidTiles = false;
						break;
					}

					FLandscapeFileInfo HeightmapInfo = HeightmapFormat->Validate(*Filename, NAME_None);
					if (HeightmapInfo.ResultCode != ELandscapeImportResult::Success)
					{
						StatusMessage = HeightmapInfo.ErrorMessage;

						if (HeightmapInfo.ResultCode == ELandscapeImportResult::Error)
						{
							bValidTiles = false;
							break;
						}
					}

					const FLandscapeFileResolution* FoundSquareResolution =
						HeightmapInfo.PossibleResolutions.FindByPredicate(
							[](const FLandscapeFileResolution& Resolution)
					{
						return Resolution.Width == Resolution.Height;
					});
					if (!FoundSquareResolution)
					{
						StatusMessage = FText::Format(LOCTEXT("TiledLandscapeImport_NotSquare", "File ({FileName}) is not square."), Arguments);
						bValidTiles = false;
						break;
					}

					if (TargetSizeX == 0)
					{
						TargetSizeX = FoundSquareResolution->Width;
						if (HeightmapInfo.DataScale.IsSet())
						{
							ImportSettings.Scale3D = HeightmapInfo.DataScale.GetValue();
							ImportSettings.Scale3D.Z *= LANDSCAPE_INV_ZSCALE;
						}
					}
					else
					{
						if (TargetSizeX != FoundSquareResolution->Width)
						{
							Arguments.Add(TEXT("Size"), FoundSquareResolution->Width);
							Arguments.Add(TEXT("TargetSize"), TargetSizeX);
							StatusMessage = FText::Format(LOCTEXT("TiledLandscapeImport_HeightmapPngTileSizeMismatch", "File ({FileName}) size ({Size}\u00D7{Size}) should match other tiles file size ({TargetSize}\u00D7{TargetSize})."), Arguments);
							bValidTiles = false;
							break;
						}
					}

					TotalLandscapeRect.Include(TileCoordinate);
					ImportSettings.TileCoordinates.Add(TileCoordinate);
				}

				if (bValidTiles)
				{
					if (SetPossibleConfigurationsForFileWidth(TargetSizeX) < 1)
					{
						FFormatNamedArguments Arguments;
						Arguments.Add(TEXT("Size"), TargetSizeX);
						StatusMessage = FText::Format(LOCTEXT("TiledLandscapeImport_HeightmapPngTileInvalidSize", "No landscape configuration found for ({Size}\u00D7{Size})."), Arguments);
						bValidTiles = false;
					}
				}
			}
		}
	}

	// Refresh combo box with active configurations
	TileConfigurationComboBox->RefreshOptions();

	return FReply::Handled();
}

FReply STiledLandscapeImportDlg::OnClickedSelectWeightmapTiles(TSharedPtr<FTiledLandscapeImportSettings::LandscapeLayerSettings> InLayerData)
{
	InLayerData->WeightmapFiles.Empty();

	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if (DesktopPlatform)
	{
		if (ParentWindow->GetNativeWindow().IsValid())
		{
			void* ParentWindowWindowHandle = ParentWindow->GetNativeWindow()->GetOSWindowHandle();

			TArray<FString> WeightmapFilesList;

			ILandscapeEditorModule& LandscapeEditorModule = FModuleManager::GetModuleChecked<ILandscapeEditorModule>("LandscapeEditor");
			const TCHAR* FileTypes = LandscapeEditorModule.GetWeightmapImportDialogTypeString();

			bool bOpened = DesktopPlatform->OpenFileDialog(
								ParentWindowWindowHandle,
								LOCTEXT("SelectWeightmapTiles", "Select weightmap tiles").ToString(),
								*FEditorDirectories::Get().GetLastDirectory(ELastDirectory::UNR),
								TEXT(""),
								FileTypes,
								EFileDialogFlags::Multiple,
								WeightmapFilesList);

			if (bOpened && WeightmapFilesList.Num())
			{
				for (FString WeighmapFile : WeightmapFilesList)
				{
					FIntPoint TileCoordinate = ExtractTileCoordinates(FPaths::GetBaseFilename(WeighmapFile));
					InLayerData->WeightmapFiles.Add(TileCoordinate, WeighmapFile);
				}

				// TODO: check if it's a valid weightmaps
			}
		}
	}
	
	return FReply::Handled();
}

bool STiledLandscapeImportDlg::IsImportEnabled() const
{
	return ImportSettings.HeightmapFileList.Num() && ImportSettings.ComponentsNum > 0;
}

FReply STiledLandscapeImportDlg::OnClickedImport()
{
	// copy weightmaps list data to an import structure  
	ImportSettings.LandscapeLayerSettingsList.Empty();

	for (int32 LayerIdx = 0; LayerIdx < LayerDataList.Num(); ++LayerIdx)
	{
		ImportSettings.LandscapeLayerSettingsList.Add(*(LayerDataList[LayerIdx].Get()));
	}

	ParentWindow->RequestDestroyWindow();
	bShouldImport = true;
	return FReply::Handled();
}

FReply STiledLandscapeImportDlg::OnClickedCancel()
{
	ParentWindow->RequestDestroyWindow();
	bShouldImport = false;

	return FReply::Handled();
}

FString STiledLandscapeImportDlg::GetLandscapeMaterialPath() const
{
	return ImportSettings.LandscapeMaterial.IsValid() ? ImportSettings.LandscapeMaterial->GetPathName() : FString("");
}

void STiledLandscapeImportDlg::OnLandscapeMaterialChanged(const FAssetData& AssetData)
{
	ImportSettings.LandscapeMaterial = Cast<UMaterialInterface>(AssetData.GetAsset());

	// pull landscape layers from a chosen material
	UpdateLandscapeLayerList();
}

int32 STiledLandscapeImportDlg::SetPossibleConfigurationsForFileWidth(int64 TargetFileWidth)
{
	int32 Idx = AllConfigurations.IndexOfByPredicate([&](const FTileImportConfiguration& A){
		return TargetFileWidth == A.SizeX;
	});

	ActiveConfigurations.Empty();
	ImportSettings.ComponentsNum = 0; // Set invalid options

	// AllConfigurations - sorted by resolution
	for (; AllConfigurations.IsValidIndex(Idx) && AllConfigurations[Idx].SizeX == TargetFileWidth; ++Idx)
	{
		TSharedPtr<FTileImportConfiguration> TileConfig = MakeShareable(new FTileImportConfiguration(AllConfigurations[Idx]));
		ActiveConfigurations.Add(TileConfig);
	}

	// Refresh combo box with active configurations
	TileConfigurationComboBox->RefreshOptions();
	// Set first configuration as active
	if (ActiveConfigurations.Num())
	{
		TileConfigurationComboBox->SetSelectedItem(ActiveConfigurations[0]);
	}

	return ActiveConfigurations.Num();
}

void STiledLandscapeImportDlg::GenerateAllPossibleTileConfigurations()
{
	AllConfigurations.Empty();
	for (int32 ComponentsNum = 1; ComponentsNum <= 32; ComponentsNum++)
	{
		for (int32 SectionsPerComponent = 1; SectionsPerComponent <= 2; SectionsPerComponent++)
		{
			for (int32 QuadsPerSection = 3; QuadsPerSection <= 8; QuadsPerSection++)
			{
				FTileImportConfiguration Entry;
				Entry.NumComponents				= ComponentsNum;
				Entry.NumSectionsPerComponent	= SectionsPerComponent;
				Entry.NumQuadsPerSection		= (1 << QuadsPerSection) - 1;
				Entry.SizeX = CalcLandscapeSquareResolution(Entry.NumComponents, Entry.NumSectionsPerComponent, Entry.NumQuadsPerSection);

				AllConfigurations.Add(Entry);
			}
		}
	}

	// Sort by resolution
	AllConfigurations.Sort([](const FTileImportConfiguration& A, const FTileImportConfiguration& B){
		if (A.SizeX == B.SizeX)
		{
			return A.NumComponents < B.NumComponents;
		}
		return A.SizeX < B.SizeX;
	});
}

FText STiledLandscapeImportDlg::GetImportSummaryText() const
{
	FTextBuilder StatusMessageBuilder;
	StatusMessageBuilder.AppendLine(StatusMessage);

	if (ImportSettings.HeightmapFileList.Num() && ImportSettings.ComponentsNum > 0)
	{
		// Tile information(num, resolution)
		const FString TilesSummary = FString::Printf(TEXT("%d - %dx%d"), ImportSettings.HeightmapFileList.Num(), ImportSettings.SizeX, ImportSettings.SizeX);
	
		// Total landscape size(NxN km)
		int32 WidthInTilesX = TotalLandscapeRect.Width() + 1;
		int32 WidthInTilesY = TotalLandscapeRect.Height() + 1;
		float WidthX = 0.00001f*ImportSettings.Scale3D.X*WidthInTilesX*ImportSettings.SizeX;
		float WidthY = 0.00001f*ImportSettings.Scale3D.Y*WidthInTilesY*ImportSettings.SizeX;
		const FString LandscapeSummary = FString::Printf(TEXT("%.3fx%.3f"), WidthX, WidthY);

		StatusMessageBuilder.AppendLine(FText::Format(
			LOCTEXT("TiledLandscapeImport_SummaryText", "{0} tiles, {1}km landscape"), 
			FText::FromString(TilesSummary),
			FText::FromString(LandscapeSummary)
			));
	}
	
	return StatusMessageBuilder.ToText();
}

FText STiledLandscapeImportDlg::GetWeightmapCountText(TSharedPtr<FTiledLandscapeImportSettings::LandscapeLayerSettings> InLayerData) const
{
	int32 NumWeighmaps = InLayerData.IsValid() ? InLayerData->WeightmapFiles.Num() : 0;
	return FText::AsNumber(NumWeighmaps);
}

ECheckBoxState STiledLandscapeImportDlg::GetLayerBlendState(TSharedPtr<FTiledLandscapeImportSettings::LandscapeLayerSettings> InLayerData) const
{
	return (InLayerData->bNoBlendWeight ? ECheckBoxState::Unchecked : ECheckBoxState::Checked);
}

void STiledLandscapeImportDlg::OnLayerBlendStateChanged(ECheckBoxState NewState, TSharedPtr<FTiledLandscapeImportSettings::LandscapeLayerSettings> InLayerData)
{
	InLayerData->bNoBlendWeight = !(NewState == ECheckBoxState::Checked);
}

FText STiledLandscapeImportDlg::GenerateConfigurationText(int32 NumComponents, int32 NumSectionsPerComponent, int32 NumQuadsPerSection) const
{
	const FString ComponentsStr = FString::Printf(TEXT("%dx%d"), NumComponents, NumComponents);
	const FString SectionsStr = FString::Printf(TEXT("%dx%d"), NumSectionsPerComponent, NumSectionsPerComponent);
	const FString QuadsStr = FString::Printf(TEXT("%dx%d"), NumQuadsPerSection, NumQuadsPerSection);
	
	return FText::Format(
		LOCTEXT("TiledLandscapeImport_ConfigurationDescFmt", "Components: {0} Sections: {1} Quads: {2}"), 
		FText::FromString(ComponentsStr),
		FText::FromString(SectionsStr),
		FText::FromString(QuadsStr)
		);
}

void STiledLandscapeImportDlg::UpdateLandscapeLayerList()
{
	TArray<FName> LayerNamesList = UE::Landscape::RetrieveTargetLayerNamesFromMaterial(ImportSettings.LandscapeMaterial.Get());
	LayerDataList.Empty();
	for (FName LayerName : LayerNamesList)
	{
		// List view data source
		TSharedPtr<FTiledLandscapeImportSettings::LandscapeLayerSettings> LayerData = MakeShareable(new FTiledLandscapeImportSettings::LandscapeLayerSettings());
		LayerData->Name = LayerName;
		LayerDataList.Add(LayerData);
	}

	LayerDataListView->RequestListRefresh();
}

#undef LOCTEXT_NAMESPACE
