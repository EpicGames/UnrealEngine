// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "LevelCollectionModel.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Input/SComboBox.h"

struct FAssetData;

/////////////////////////////////////////////////////
// STiledLandscapeImportDlg
// 
class STiledLandscapeImportDlg
	: public SCompoundWidget
{
public:
	/** */
	struct FTileImportConfiguration
	{
		int32 SizeX;
		int32 NumComponents;
		int32 NumSectionsPerComponent;
		int32 NumQuadsPerSection;
	};

	SLATE_BEGIN_ARGS( STiledLandscapeImportDlg )
		{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<SWindow> InParentWindow);

	/** */
	bool ShouldImport() const;	
	const FTiledLandscapeImportSettings& GetImportSettings() const;

private:
	/** */
	TSharedRef<SWidget> HandleTileConfigurationComboBoxGenarateWidget(TSharedPtr<FTileImportConfiguration> InItem) const;
	FText				GetTileConfigurationText() const;

	/** */
	TSharedRef<ITableRow> OnGenerateWidgetForLayerDataListView(TSharedPtr<FTiledLandscapeImportSettings::LandscapeLayerSettings> InLayerData, const TSharedRef<STableViewBase>& OwnerTable);

	/** */
	TOptional<float> GetScaleX() const;
	TOptional<float> GetScaleY() const;
	TOptional<float> GetScaleZ() const;
	void OnSetScale(float InValue, ETextCommit::Type CommitType, int32 InAxis);

	/** */
	TOptional<int32> GetTileOffsetX() const;
	TOptional<int32> GetTileOffsetY() const;
	void SetTileOffsetX(int32 InValue);
	void SetTileOffsetY(int32 InValue);

	/** */
	ECheckBoxState GetFlipYAxisState() const;
	void OnFlipYAxisStateChanged(ECheckBoxState NewState);

	/** */
	void OnSetImportConfiguration(TSharedPtr<FTileImportConfiguration> InTileConfig, ESelectInfo::Type SelectInfo);

	/** */
	bool IsImportEnabled() const;

	/** */
	FReply OnClickedSelectHeightmapTiles();
	FReply OnClickedSelectWeightmapTiles(TSharedPtr<FTiledLandscapeImportSettings::LandscapeLayerSettings> InLayerData);
	FReply OnClickedImport();
	FReply OnClickedCancel();

	/** */
	FString GetLandscapeMaterialPath() const;
	void OnLandscapeMaterialChanged(const FAssetData& AssetData);

	/** */
	FText GetImportSummaryText() const;
	FText GetWeightmapCountText(TSharedPtr<FTiledLandscapeImportSettings::LandscapeLayerSettings> InLayerData) const; 

	/** */
	ECheckBoxState GetLayerBlendState(TSharedPtr<FTiledLandscapeImportSettings::LandscapeLayerSettings> InLayerData) const;
	void OnLayerBlendStateChanged(ECheckBoxState NewState, TSharedPtr<FTiledLandscapeImportSettings::LandscapeLayerSettings> InLayerData);

	/** */
	int32 SetPossibleConfigurationsForFileWidth(int64 TargetFileWidth);

	/** */
	void GenerateAllPossibleTileConfigurations();

	/** */
	FText GenerateConfigurationText(int32 NumComponents, int32 NumSectionsPerComponent,	int32 NumQuadsPerSection) const;

	/** */
	void UpdateLandscapeLayerList();

private:
	/** */
	bool bShouldImport;

	/** */
	mutable FText StatusMessage;

	/** */
	TSharedPtr<SWindow> ParentWindow;

	/** */
	TSharedPtr<SComboBox<TSharedPtr<FTileImportConfiguration>>> TileConfigurationComboBox;

	TSharedPtr<SListView<TSharedPtr<FTiledLandscapeImportSettings::LandscapeLayerSettings>>>	LayerDataListView;
	TArray<TSharedPtr<FTiledLandscapeImportSettings::LandscapeLayerSettings>>					LayerDataList;

	/** */
	FTiledLandscapeImportSettings ImportSettings;

	/** */
	FIntRect TotalLandscapeRect;

	TArray<FTileImportConfiguration> AllConfigurations;
	TArray<TSharedPtr<FTileImportConfiguration>> ActiveConfigurations;
};

