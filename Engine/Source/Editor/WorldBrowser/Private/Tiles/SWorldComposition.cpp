// Copyright Epic Games, Inc. All Rights Reserved.
#include "Tiles/SWorldComposition.h"
#include "Layout/ArrangedChildren.h"
#include "Rendering/DrawElements.h"
#include "Widgets/SOverlay.h"
#include "Engine/GameViewportClient.h"
#include "Misc/PackageName.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleManager.h"
#include "Layout/WidgetPath.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Images/SImage.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SButton.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "Editor.h"
#include "WorldBrowserModule.h"
#include "SNodePanel.h"
#include "LevelCollectionCommands.h"

#include "Tiles/SWorldTileItem.h"
#include "Tiles/SWorldLayers.h"
#include "Tiles/WorldTileThumbnails.h"

#define LOCTEXT_NAMESPACE "WorldBrowser"

//----------------------------------------------------------------
//
//
//----------------------------------------------------------------
struct FWorldZoomLevelsContainer 
	: public FZoomLevelsContainer
{
	float	GetZoomAmount(int32 InZoomLevel) const override
	{
		return 1.f/FMath::Square(GetNumZoomLevels() - InZoomLevel + 1)*2.f;
	}

	int32 GetNearestZoomLevel( float InZoomAmount ) const override
	{
		for (int32 ZoomLevelIndex=0; ZoomLevelIndex < GetNumZoomLevels(); ++ZoomLevelIndex)
		{
			if (InZoomAmount <= GetZoomAmount(ZoomLevelIndex))
			{
				return ZoomLevelIndex;
			}
		}

		return GetDefaultZoomLevel();
	}

	FText GetZoomText(int32 InZoomLevel) const override
	{
		return FText::AsNumber(GetZoomAmount(InZoomLevel));
	}

	int32	GetNumZoomLevels() const override
	{
		return 300;
	}

	int32	GetDefaultZoomLevel() const override
	{
		return GetNumZoomLevels() - 10;
	}

	EGraphRenderingLOD::Type GetLOD(int32 InZoomLevel) const override
	{
		return EGraphRenderingLOD::DefaultDetail;
	}
};

//----------------------------------------------------------------
//
//
//----------------------------------------------------------------

namespace WorldCompositionGridDefs
{
	static const float ZoomPaddingFraction = 0.125f;
}

class SWorldCompositionGrid 
	: public SNodePanel
{
public:
	SLATE_BEGIN_ARGS(SWorldCompositionGrid) {}
		SLATE_ARGUMENT(TSharedPtr<FWorldTileCollectionModel>, InWorldModel)
	SLATE_END_ARGS()

	SWorldCompositionGrid()
		: CommandList(MakeShareable(new FUICommandList))
		, bHasScrollByRequest(false)
		, bIsFirstTickCall(true)
		, bHasNodeInteraction(true)
		, BoundsSnappingDistance(20.f)
	{
	}

	~SWorldCompositionGrid()
	{
		WorldModel->SelectionChanged.RemoveAll(this);
		WorldModel->CollectionChanged.RemoveAll(this);

		FCoreDelegates::PreWorldOriginOffset.RemoveAll(this);
	}

	void Construct(const FArguments& InArgs)
	{
		SetZoomLevelsContainer<FWorldZoomLevelsContainer>();

		SNodePanel::Construct();

		bTeleportInsteadOfScrollingWhenZoomingToFit = true;

		// otherwise tiles will be drawn outside of this widget area
		SetClipping(EWidgetClipping::ClipToBounds);

		//
		WorldModel = InArgs._InWorldModel;
		bUpdatingSelection = false;
	
		WorldModel->SelectionChanged.AddSP(this, &SWorldCompositionGrid::OnUpdateSelection);
		WorldModel->CollectionChanged.AddSP(this, &SWorldCompositionGrid::RefreshView);
		SelectionManager.OnSelectionChanged.BindSP(this, &SWorldCompositionGrid::OnSelectionChanged);

		FCoreDelegates::PreWorldOriginOffset.AddSP(this, &SWorldCompositionGrid::PreWorldOriginOffset);

		BindCommands();

		ThumbnailCollection = MakeShareable(new FTileThumbnailCollection());
	
		RefreshView();
	}
	
	/**  Add specified item to the grid view */
	void AddItem(TSharedPtr<FWorldTileModel> LevelModel)
	{
		auto NewNode = SNew(SWorldTileItem)
							.InWorldModel(WorldModel)
							.InItemModel(LevelModel)
							.InThumbnailCollection(ThumbnailCollection);
	
		AddGraphNode(NewNode);
	}
	
	/**  Remove specified item from the grid view */
	void RemoveItem(TSharedPtr<FLevelModel> LevelModel)
	{
		TSharedRef<SNode>* pItem = NodeToWidgetLookup.Find(LevelModel->GetNodeObject());
		if (pItem == NULL)
		{
			return;
		}

		Children.Remove(*pItem);
		VisibleChildren.Remove(*pItem);
		NodeToWidgetLookup.Remove(LevelModel->GetNodeObject());
	}
		
	/**  Updates all the items in the grid view */
	void RefreshView()
	{
		RemoveAllNodes();

		FLevelModelList AllLevels = WorldModel->GetAllLevels();
		for (auto It = AllLevels.CreateConstIterator(); It; ++It)
		{
			AddItem(StaticCastSharedPtr<FWorldTileModel>(*It));
		}
	}
		
	/**  SWidget interface */
	void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override
	{
		SNodePanel::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

		// fit view to selection on first tick
		if (bIsFirstTickCall)
		{
			bIsFirstTickCall = false;
			FitToSelection_Executed();
		}

		FVector2D CursorPosition = FSlateApplication::Get().GetCursorPos();

		// Update cached variables
		WorldMouseLocation = CursorToWorldPosition(AllottedGeometry, CursorPosition);
		WorldMarqueeSize = Marquee.Rect.GetSize()/AllottedGeometry.Scale;
			
		// Update streaming preview data
		const bool bShowPotentiallyVisibleLevels = FSlateApplication::Get().GetModifierKeys().IsAltDown() && 
													AllottedGeometry.IsUnderLocation(CursorPosition);
	
		WorldModel->UpdateStreamingPreview(WorldMouseLocation, bShowPotentiallyVisibleLevels);

		// deferred scroll by requests
		if (bHasScrollByRequest)
		{
			bHasScrollByRequest = false;
			SetViewOffset(GetViewOffset() + RequestedScrollByValue);
		}
	}
	
	/**  SWidget interface */
	virtual void OnArrangeChildren( const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren ) const override
	{
		for (int32 ChildIndex=0; ChildIndex < VisibleChildren.Num(); ++ChildIndex)
		{
			const auto Child = StaticCastSharedRef<SWorldTileItem>(VisibleChildren[ChildIndex]);
			const EVisibility ChildVisibility = Child->GetVisibility();

			if (ArrangedChildren.Accepts(ChildVisibility))
			{
				FVector2f ChildPos = Child->GetPosition2f();
					
				ArrangedChildren.AddWidget(ChildVisibility,
					AllottedGeometry.MakeChild(Child,
					ChildPos - GetViewOffset(),
					Child->GetDesiredSize(), GetZoomAmount()
					));
			}
		}
	}

	/**  SWidget interface */
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled ) const override
	{
		// First paint the background
		{
			LayerId = PaintBackground(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId);
			LayerId++;
		}

		FArrangedChildren ArrangedChildren(EVisibility::Visible);
		ArrangeChildren(AllottedGeometry, ArrangedChildren);

		// Draw the child nodes

		// When drawing a marquee, need a preview of what the selection will be.
		const auto* SelectionToVisualize = &ObjectPtrDecay(SelectionManager.SelectedNodes);
		decltype(SelectionManager.SelectedNodes) SelectionPreview;
		if (Marquee.IsValid())
		{			
			ApplyMarqueeSelection(Marquee, ObjectPtrDecay(SelectionManager.SelectedNodes), SelectionPreview);
			SelectionToVisualize = &ObjectPtrDecay(SelectionPreview);
		}
	
		int32 NodesLayerId = LayerId;

		for (int32 ChildIndex = 0; ChildIndex < ArrangedChildren.Num(); ++ChildIndex)
		{
			FArrangedWidget& CurWidget = ArrangedChildren[ChildIndex];
			TSharedRef<SWorldTileItem> ChildNode = StaticCastSharedRef<SWorldTileItem>(CurWidget.Widget);
		
			ChildNode->bAffectedByMarquee = SelectionToVisualize->Contains(ChildNode->GetObjectBeingDisplayed());
			LayerId = CurWidget.Widget->Paint(Args.WithNewParent(this), CurWidget.Geometry, MyCullingRect, OutDrawElements, NodesLayerId, InWidgetStyle, ShouldBeEnabled(bParentEnabled));
			ChildNode->bAffectedByMarquee = false;
		}
		
		// Draw editable world bounds
		if (!WorldModel->IsSimulating())
		{
			float ScreenSpaceSize = FLevelCollectionModel::EditableAxisLength()*GetZoomAmount()*2.f;
			FVector2f PaintSize = FVector2f(ScreenSpaceSize, ScreenSpaceSize);
			FVector2f PaintPosition = GraphCoordToPanelCoord(FVector2f::ZeroVector) - (PaintSize*0.5f);
			float Scale = 0.2f; // Scale down drawing border
			FSlateLayoutTransform LayoutTransform(Scale, AllottedGeometry.GetAccumulatedLayoutTransform().GetTranslation() + PaintPosition);
			FSlateRenderTransform SlateRenderTransform(Scale, UE::Slate::CastToVector2f(AllottedGeometry.GetAccumulatedRenderTransform().GetTranslation()) + PaintPosition);
			FPaintGeometry EditableArea(LayoutTransform, SlateRenderTransform, PaintSize/Scale, !SlateRenderTransform.IsIdentity());

			FLinearColor PaintColor = FLinearColor::Yellow;
			PaintColor.A = 0.4f;

			FSlateDrawElement::MakeBox(
				OutDrawElements,
				++LayerId,
				EditableArea,
				FAppStyle::GetBrush(TEXT("Graph.CompactNode.ShadowSelected")),
				ESlateDrawEffect::None,
				PaintColor
				);
		}
		
		// Draw the marquee selection rectangle
		PaintMarquee(AllottedGeometry, MyCullingRect, OutDrawElements, ++LayerId);

		// Draw the software cursor
		PaintSoftwareCursor(AllottedGeometry, MyCullingRect, OutDrawElements, ++LayerId);

		if(WorldModel->IsSimulating())
		{
			// Draw a surrounding indicator when PIE is active, to make it clear that the graph is read-only, etc...
			FSlateDrawElement::MakeBox(
				OutDrawElements,
				LayerId,
				AllottedGeometry.ToPaintGeometry(),
				FAppStyle::GetBrush(TEXT("Graph.PlayInEditor"))
				);
		}

		// Draw observer location
		{
			FVector ObserverPosition;
			FRotator ObserverRotation;
			if (WorldModel->GetObserverView(ObserverPosition, ObserverRotation))
			{
				FVector2f ObserverPositionScreen = GraphCoordToPanelCoord(FVector2f(UE_REAL_TO_FLOAT(ObserverPosition.X), UE_REAL_TO_FLOAT(ObserverPosition.Y)));
				const FSlateBrush* CameraImage = FAppStyle::GetBrush(TEXT("WorldBrowser.SimulationViewPosition"));
	
				//AllottedGeometry.GetAccumulatedRenderTransform();
				//FSlateLayoutTransform LayoutTransform(Scale, AllottedGeometry.GetAccumulatedLayoutTransform().GetTranslation() - InflateAmount);
				//FSlateRenderTransform SlateRenderTransform(Scale, AllottedGeometry.GetAccumulatedRenderTransform().GetTranslation() - InflateAmount);

				FPaintGeometry PaintGeometry = AllottedGeometry.ToPaintGeometry( 
					CameraImage->ImageSize,
					FSlateLayoutTransform(ObserverPositionScreen - CameraImage->ImageSize*0.5f)
				);

				FSlateDrawElement::MakeRotatedBox(
					OutDrawElements,
					++LayerId,
					PaintGeometry,
					CameraImage,
					ESlateDrawEffect::None,
					FMath::DegreesToRadians(ObserverRotation.Yaw),
					CameraImage->ImageSize*0.5f,
					FSlateDrawElement::RelativeToElement
					);
			}

			FVector PlayerPosition;
			FRotator PlayerRotation;
			if (WorldModel->GetPlayerView(PlayerPosition, PlayerRotation))
			{
				FVector2f PlayerPositionScreen = GraphCoordToPanelCoord(FVector2f(UE_REAL_TO_FLOAT(PlayerPosition.X), UE_REAL_TO_FLOAT(PlayerPosition.Y)));
				const FSlateBrush* CameraImage = FAppStyle::GetBrush(TEXT("WorldBrowser.SimulationViewPosition"));
	
				FPaintGeometry PaintGeometry = AllottedGeometry.ToPaintGeometry(
					CameraImage->ImageSize,
					FSlateLayoutTransform(PlayerPositionScreen - CameraImage->ImageSize*0.5f)
					);

				FSlateDrawElement::MakeRotatedBox(
					OutDrawElements,
					++LayerId,
					PaintGeometry,
					CameraImage,
					ESlateDrawEffect::None,
					FMath::DegreesToRadians(PlayerRotation.Yaw),
					CameraImage->ImageSize*0.5f,
					FSlateDrawElement::RelativeToElement,
					FLinearColor(FColorList::Orange)
					);
			}

		}

		LayerId = PaintScaleRuler(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId);
		return LayerId;
	}
		
	/** SWidget interface */
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		const bool bIsRightMouseButtonDown = MouseEvent.IsMouseButtonDown( EKeys::RightMouseButton );
		const bool bIsLeftMouseButtonDown = MouseEvent.IsMouseButtonDown( EKeys::LeftMouseButton );
		const bool bIsMiddleMouseButtonDown = MouseEvent.IsMouseButtonDown(EKeys::MiddleMouseButton);

		PastePosition = PanelCoordToGraphCoord( MyGeometry.AbsoluteToLocal( MouseEvent.GetScreenSpacePosition() ) );

		if ( this->HasMouseCapture() )
		{
			const FVector2D CursorDelta = MouseEvent.GetCursorDelta();
			// Track how much the mouse moved since the mouse down.
			TotalMouseDelta += CursorDelta.Size();

			// Zooming - drop through to default behaviour
			if(bIsRightMouseButtonDown && (bIsLeftMouseButtonDown || bIsMiddleMouseButtonDown || FSlateApplication::Get().GetModifierKeys().IsAltDown() || FSlateApplication::Get().IsUsingTrackpad()))
			{
				return SNodePanel::OnMouseMove(MyGeometry, MouseEvent);
			}
			else if (bIsRightMouseButtonDown || bIsMiddleMouseButtonDown)
			{
				FReply ReplyState = FReply::Handled();

				if( !CursorDelta.IsZero() )
				{
					bShowSoftwareCursor = true;
				}

				// Panning and mouse is outside of panel? Pasting should just go to the screen center.
				PastePosition = PanelCoordToGraphCoord( 0.5f * MyGeometry.GetLocalSize() );

				this->bIsPanning = true;
				SetViewOffset(GetViewOffset() - CursorDelta / GetZoomAmount());

				return ReplyState;
			}
			else if (bIsLeftMouseButtonDown)
			{
				TSharedPtr<SNode> NodeBeingDragged = NodeUnderMousePtr.Pin();

				if ( GetIsEditable() )
				{
					// Update the amount to pan panel
					UpdateViewOffset(MyGeometry, MouseEvent.GetScreenSpacePosition());

					const bool bCursorInDeadZone = TotalMouseDelta <= FSlateApplication::Get().GetDragTriggerDistance();

					if ( NodeBeingDragged.IsValid() )
					{
						if ( !bCursorInDeadZone )
						{
							// Note, NodeGrabOffset() comes from the node itself, so it's already scaled correctly.
							FVector2f AnchorNodeNewPos = PanelCoordToGraphCoord( MyGeometry.AbsoluteToLocal( MouseEvent.GetScreenSpacePosition() ) ) - NodeGrabOffset;

							// Dragging an unselected node automatically selects it.
							SelectionManager.StartDraggingNode(NodeBeingDragged->GetObjectBeingDisplayed(), MouseEvent);

							// Move all the selected nodes.
							{
								const FVector2f AnchorNodeOldPos = NodeBeingDragged->GetPosition2f();
								const FVector2f DeltaPos = AnchorNodeNewPos - AnchorNodeOldPos;
								if (DeltaPos.SizeSquared() > FMath::Square(KINDA_SMALL_NUMBER))
								{
									MoveSelectedNodes(NodeBeingDragged, AnchorNodeNewPos);
								}
							}
						}

						return FReply::Handled();
					}
				}

				if ( !NodeBeingDragged.IsValid() )
				{
					// We are marquee selecting
					const FVector2D GraphMousePos = PanelCoordToGraphCoord( MyGeometry.AbsoluteToLocal( MouseEvent.GetScreenSpacePosition() ) );
					Marquee.Rect.UpdateEndPoint(GraphMousePos);

					FindNodesAffectedByMarquee( /*out*/ Marquee.AffectedNodes );
					return FReply::Handled();
				}
			}
		}

		return FReply::Unhandled();
	}

	/** @return Size of a marquee rectangle in world space */
	FVector2D GetMarqueeWorldSize() const
	{
		return WorldMarqueeSize;
	}

	/** @return Mouse cursor position in world space */
	FVector2D GetMouseWorldLocation() const
	{
		return WorldMouseLocation;
	}

protected:
	/** Bind commands */
	void BindCommands()
	{
		FUICommandList& CommandListRef = *CommandList;

		const FLevelCollectionCommands& Commands = FLevelCollectionCommands::Get();

		CommandListRef.MapAction(Commands.FitToSelection,
			FExecuteAction::CreateSP(this, &SWorldCompositionGrid::FitToSelection_Executed));
	}
	/**  Draws background for grid view */
	uint32 PaintBackground(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, uint32 LayerId) const
	{
		FVector2f ScreenWorldOrigin = GraphCoordToPanelCoord(FVector2f::ZeroVector);
		FSlateRect ScreenRect(FVector2f::ZeroVector, AllottedGeometry.GetLocalSize());
	
		// World Y-axis
		if (ScreenWorldOrigin.X > ScreenRect.Left &&
			ScreenWorldOrigin.X < ScreenRect.Right)
		{
			TArray<FVector2f> LinePoints;
			LinePoints.Add(FVector2f(ScreenWorldOrigin.X, ScreenRect.Top));
			LinePoints.Add(FVector2f(ScreenWorldOrigin.X, ScreenRect.Bottom));

			FLinearColor YAxisColor = FLinearColor::Green;
			YAxisColor.A = 0.4f;
		
			FSlateDrawElement::MakeLines( 
				OutDrawElements,
				LayerId,
				AllottedGeometry.ToPaintGeometry(),
				LinePoints,
				ESlateDrawEffect::None,
				YAxisColor);
		}

		// World X-axis
		if (ScreenWorldOrigin.Y > ScreenRect.Top &&
			ScreenWorldOrigin.Y < ScreenRect.Bottom)
		{
			TArray<FVector2f> LinePoints;
			LinePoints.Add(FVector2f(ScreenRect.Left, ScreenWorldOrigin.Y));
			LinePoints.Add(FVector2f(ScreenRect.Right, ScreenWorldOrigin.Y));

			FLinearColor XAxisColor = FLinearColor::Red;
			XAxisColor.A = 0.4f;
		
			FSlateDrawElement::MakeLines( 
				OutDrawElements,
				LayerId,
				AllottedGeometry.ToPaintGeometry(),
				LinePoints,
				ESlateDrawEffect::None,
				XAxisColor);
		}

		return LayerId + 1;
	}

	/**  Draws current scale */
	uint32 PaintScaleRuler(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, uint32 LayerId) const
	{
		const float	ScaleRulerLength = 100.f; // pixels
		TArray<FVector2f> LinePoints;
		LinePoints.Add(FVector2f::ZeroVector);
		LinePoints.Add(FVector2f::ZeroVector + FVector2f(ScaleRulerLength, 0.f));
	
		FSlateDrawElement::MakeLines( 
			OutDrawElements,
			LayerId,
			AllottedGeometry.ToOffsetPaintGeometry(FVector2f(10.0f, 40.0f)),
			LinePoints,
			ESlateDrawEffect::None,
			FColor(200, 200, 200));

		const float UnitsInRuler = ScaleRulerLength/GetZoomAmount();// Pixels to world units
		const int32 UnitsInMeter = 100;
		const int32 UnitsInKilometer = UnitsInMeter*1000;
	
		FString RulerText;
		if (UnitsInRuler > UnitsInKilometer) // in kilometers
		{
			RulerText = FString::Printf(TEXT("%.2f km"), UnitsInRuler/UnitsInKilometer);
		}
		else // in meters
		{
			RulerText = FString::Printf(TEXT("%.2f m"), UnitsInRuler/UnitsInMeter);
		}
	
		FSlateDrawElement::MakeText(
			OutDrawElements,
			LayerId,
			AllottedGeometry.ToOffsetPaintGeometry(FVector2f(10.0f, 27.0f)),
			RulerText,
			FAppStyle::GetFontStyle("NormalFont"),
			ESlateDrawEffect::None,
			FColor(200, 200, 200));
		
		return LayerId + 1;
	}
		
	/**  SWidget interface */
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override
	{
		if (CommandList->ProcessCommandBindings(InKeyEvent))
		{
			return FReply::Handled();
		}
	
		if (WorldModel->GetCommandList()->ProcessCommandBindings(InKeyEvent))
		{
			return FReply::Handled();
		}

		return SNodePanel::OnKeyDown(MyGeometry, InKeyEvent);
	}

	/**  SWidget interface */
	bool SupportsKeyboardFocus() const override
	{
		return true;
	}
	
	/**  SNodePanel interface */
	TSharedPtr<SWidget> OnSummonContextMenu(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (WorldModel->IsReadOnly())
		{
			return SNullWidget::NullWidget;
		}

		FArrangedChildren ArrangedChildren(EVisibility::Visible);
		const bool bUpdateVisibilityAttributes = true;
		ArrangeChildren(MyGeometry, ArrangedChildren, bUpdateVisibilityAttributes);

		const int32 NodeUnderMouseIndex = SWidget::FindChildUnderMouse( ArrangedChildren, MouseEvent );
		if (NodeUnderMouseIndex != INDEX_NONE)
		{
			// PRESSING ON A NODE!
			const FArrangedWidget& NodeGeometry = ArrangedChildren[NodeUnderMouseIndex];
			const FVector2f MousePositionInNode = NodeGeometry.Geometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
			TSharedRef<SNode> NodeWidgetUnderMouse = StaticCastSharedRef<SNode>( NodeGeometry.Widget );

			if (NodeWidgetUnderMouse->CanBeSelected(MousePositionInNode))
			{
				if (!SelectionManager.IsNodeSelected(NodeWidgetUnderMouse->GetObjectBeingDisplayed()))
				{
					SelectionManager.SelectSingleNode(NodeWidgetUnderMouse->GetObjectBeingDisplayed());
				}
			}
		}
		else
		{
			SelectionManager.ClearSelectionSet();
		}
	
		// Summon context menu
		FMenuBuilder MenuBuilder(true, WorldModel->GetCommandList());
		WorldModel->BuildWorldCompositionMenu(MenuBuilder);
		TSharedPtr<SWidget> MenuWidget = MenuBuilder.MakeWidget();

		FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();

		FSlateApplication::Get().PushMenu(
			AsShared(),
			WidgetPath,
			MenuWidget.ToSharedRef(),
			MouseEvent.GetScreenSpacePosition(),
			FPopupTransitionEffect( FPopupTransitionEffect::ContextMenu )
			);

		return MenuWidget;
	}
	
	/**  SNodePanel interface */
	virtual void PopulateVisibleChildren(const FGeometry& AllottedGeometry) override
	{
		VisibleChildren.Empty();

		FSlateRect PanelRect(FVector2D(0, 0), AllottedGeometry.GetLocalSize());
		FVector2D ViewStartPos = PanelCoordToGraphCoord(FVector2D(PanelRect.Left, PanelRect.Top));
		FVector2D ViewEndPos = PanelCoordToGraphCoord(FVector2D(PanelRect.Right, PanelRect.Bottom));
		FSlateRect ViewRect(ViewStartPos, ViewEndPos);

		for (int32 ChildIndex=0; ChildIndex < Children.Num(); ++ChildIndex)
		{
			const auto Child = StaticCastSharedRef<SWorldTileItem>(Children[ChildIndex]);
			const auto LevelModel = Child->GetLevelModel();
			if (LevelModel->IsVisibleInCompositionView())
			{
				FSlateRect ChildRect = Child->GetItemRect();
				FVector2D ChildSize = ChildRect.GetSize();

				if (ChildSize.X > 0.f && 
					ChildSize.Y > 0.f && 
					FSlateRect::DoRectanglesIntersect(ChildRect, ViewRect))
				{
					VisibleChildren.Add(Child);
				}
			}
		}

		// Sort tiles such that smaller and selected tiles will be drawn on top of other tiles
		struct FVisibleTilesSorter
		{
			FVisibleTilesSorter(const FLevelCollectionModel& InWorldModel) : WorldModel(InWorldModel)
			{}
			bool operator()(const TSharedRef<SNodePanel::SNode>& A,
							const TSharedRef<SNodePanel::SNode>& B) const
			{
				TSharedRef<SWorldTileItem> ItemA = StaticCastSharedRef<SWorldTileItem>(A);
				TSharedRef<SWorldTileItem> ItemB = StaticCastSharedRef<SWorldTileItem>(B);
				return WorldModel.CompareLevelsZOrder(ItemA->GetLevelModel(), ItemB->GetLevelModel());
			}
			const FLevelCollectionModel& WorldModel;
		};

		VisibleChildren.Sort(FVisibleTilesSorter(*WorldModel.Get()));
	}

	/**  SNodePanel interface */
	virtual void OnBeginNodeInteraction(const TSharedRef<SNode>& InNodeToDrag, const FVector2f& GrabOffset) override
	{
		bHasNodeInteraction = true;
		SNodePanel::OnBeginNodeInteraction(InNodeToDrag, GrabOffset);
	}
	
	/**  SNodePanel interface */
	virtual void OnEndNodeInteraction(const TSharedRef<SNode>& InNodeDragged) override
	{
		const SWorldTileItem& Item = static_cast<const SWorldTileItem&>(InNodeDragged.Get());
		if (Item.IsItemEditable() && !WorldModel->IsLockTilesLocationEnabled())
		{
			FVector2D AbsoluteDelta = Item.GetLevelModel()->GetLevelTranslationDelta();
			FIntPoint IntAbsoluteDelta = FIntPoint(AbsoluteDelta.X, AbsoluteDelta.Y);

			// Reset stored translation delta to 0
			WorldModel->UpdateTranslationDelta(WorldModel->GetSelectedLevels(), FVector2D::ZeroVector, false, 0.f);
	
			// In case we have non zero dragging delta, translate selected levels 
			if (IntAbsoluteDelta != FIntPoint::ZeroValue)
			{
				WorldModel->TranslateLevels(WorldModel->GetSelectedLevels(), IntAbsoluteDelta);
			}
		}
	
		bHasNodeInteraction = false;

		SNodePanel::OnEndNodeInteraction(InNodeDragged);
	}

	/** Handles selection changes in the grid view */
	void OnSelectionChanged(const FGraphPanelSelectionSet& SelectedNodes)
	{
		if (bUpdatingSelection)
		{
			return;
		}
	
		bUpdatingSelection = true;
		FLevelModelList SelectedLevels;
	
		for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
		{
			TSharedRef<SNode>* pWidget = NodeToWidgetLookup.Find(*NodeIt);
			if (pWidget != NULL)
			{
				TSharedRef<SWorldTileItem> Item = StaticCastSharedRef<SWorldTileItem>(*pWidget);
				SelectedLevels.Add(Item->GetLevelModel());
			}
		}
	
		WorldModel->SetSelectedLevels(SelectedLevels);
		bUpdatingSelection = false;
	}
	
	/** Handles selection changes in data source */
	void OnUpdateSelection()
	{
		if (bUpdatingSelection)
		{
			return;
		}

		bUpdatingSelection = true;

		SelectionManager.ClearSelectionSet();
		FLevelModelList SelectedLevels = WorldModel->GetSelectedLevels();
		for (auto It = SelectedLevels.CreateConstIterator(); It; ++It)
		{
			SelectionManager.SetNodeSelection((*It)->GetNodeObject(), true);
		}

		if (SelectionManager.AreAnyNodesSelected())
		{
			FVector2f MinCorner, MaxCorner;
			if (GetBoundsForNodes(true, MinCorner, MaxCorner, 0.f))
			{
				FSlateRect SelectionRect = FSlateRect(GraphCoordToPanelCoord(MinCorner), GraphCoordToPanelCoord(MaxCorner));
				FSlateRect PanelRect = FSlateRect(FVector2f::ZeroVector, CachedGeometry.GetLocalSize());
				bool bIsVisible = FSlateRect::DoRectanglesIntersect(PanelRect, SelectionRect);
				if (!bIsVisible)
				{
					FVector2f TargetPosition = MaxCorner/2.f + MinCorner/2.f;
					FitToSelection_Executed();
				}
			}
		}
		bUpdatingSelection = false;
	}

	/** Delegate callback: world origin is going to be moved. */
	void PreWorldOriginOffset(UWorld* InWorld, FIntVector InSrcOrigin, FIntVector InDstOrigin)
	{
		if (InWorld && (WorldModel->GetWorld() == InWorld || WorldModel->GetSimulationWorld() == InWorld))		
		{
			FIntVector Offset = InDstOrigin - InSrcOrigin;
			RequestScrollBy(-FVector2D(Offset.X, Offset.Y));
		}
	}
	
	/** Handles new item added to data source */
	void OnNewItemAdded(TSharedPtr<FLevelModel> NewItem)
	{
		RefreshView();
	}

	/**  FitToSelection command handler */
	void FitToSelection_Executed()
	{
		FVector2f MinCorner, MaxCorner;
		if (GetBoundsForNodes(true, MinCorner, MaxCorner, 0.f))
		{
			// Calculate relative zoom padding, then get the padded bounds
			ZoomPadding = FMath::Abs((MinCorner - MaxCorner).GetMax()) * WorldCompositionGridDefs::ZoomPaddingFraction;
			if (GetBoundsForNodes(true, MinCorner, MaxCorner, ZoomPadding))
			{
				ZoomToTarget(MinCorner, MaxCorner);
			}
		}
	}
		
	/**  @returns Whether any of the levels are selected */
	bool AreAnyItemsSelected() const
	{
		return SelectionManager.AreAnyNodesSelected();
	}

	void RequestScrollBy(FVector2D InDelta)
	{
		bHasScrollByRequest = true;
		RequestedScrollByValue = InDelta;
	}
	
	/** Handlers for moving items using arrow keys */
	void MoveLevelLeft_Executed()
	{
		if (!bHasNodeInteraction)
		{
			WorldModel->TranslateLevels(WorldModel->GetSelectedLevels(), FIntPoint(-1, 0));
		}
	}

	void MoveLevelRight_Executed()
	{
		if (!bHasNodeInteraction)
		{
			WorldModel->TranslateLevels(WorldModel->GetSelectedLevels(), FIntPoint(+1, 0));
		}
	}

	void MoveLevelUp_Executed()
	{
		if (!bHasNodeInteraction)
		{
			WorldModel->TranslateLevels(WorldModel->GetSelectedLevels(), FIntPoint(0, -1));
		}
	}

	void MoveLevelDown_Executed()
	{
		if (!bHasNodeInteraction)
		{
			WorldModel->TranslateLevels(WorldModel->GetSelectedLevels(), FIntPoint(0, +1));
		}
	}	
	
	/** Moves selected nodes by specified offset */
	void MoveSelectedNodes(const TSharedPtr<SNode>& InNodeToDrag, FVector2f NewPosition)
	{
		auto ItemDragged = StaticCastSharedPtr<SWorldTileItem>(InNodeToDrag);
	
		if (ItemDragged->IsItemEditable() && !WorldModel->IsLockTilesLocationEnabled())
		{
			// Current translation snapping value
			float SnappingDistanceWorld = 0.f;
			const bool bBoundsSnapping = (FSlateApplication::Get().GetModifierKeys().IsControlDown() == false);
			if (bBoundsSnapping)
			{
				SnappingDistanceWorld = BoundsSnappingDistance/GetZoomAmount();
			}
			else if (GetDefault<ULevelEditorViewportSettings>()->GridEnabled)
			{
				SnappingDistanceWorld = GEditor->GetGridSize();
			}
		
			FVector2f StartPosition = ItemDragged->GetPosition2f() - UE::Slate::CastToVector2f(ItemDragged->GetLevelModel()->GetLevelTranslationDelta());
			FVector2f AbsoluteDelta = NewPosition - StartPosition;

			WorldModel->UpdateTranslationDelta(WorldModel->GetSelectedLevels(), FVector2D(AbsoluteDelta), bBoundsSnapping, SnappingDistanceWorld);
		}
	}

	/**  Converts cursor absolute position to the world position */
	FVector2D CursorToWorldPosition(const FGeometry& InGeometry, FVector2D InAbsoluteCursorPosition)
	{
		FVector2D ViewSpacePosition = (InAbsoluteCursorPosition - FVector2D(InGeometry.AbsolutePosition))/InGeometry.Scale;
		return PanelCoordToGraphCoord(ViewSpacePosition);
	}

private:
	/** Levels data list to display*/
	TSharedPtr<FWorldTileCollectionModel>	WorldModel;

	/** Geometry cache */
	mutable FVector2D						CachedAllottedGeometryScaledSize;

	bool									bUpdatingSelection;
	TArray<FIntRect>						OccupiedCells;
	const TSharedRef<FUICommandList>		CommandList;

	bool									bHasScrollByRequest;
	FVector2D								RequestedScrollByValue;

	bool									bIsFirstTickCall;
	// Is user interacting with a node now
	bool									bHasNodeInteraction;

	// Snapping distance in screen units for a tile bounds
	float									BoundsSnappingDistance;

	//
	// Mouse location in the world
	FVector2D								WorldMouseLocation;
	// Current marquee rectangle size in world units
	FVector2D								WorldMarqueeSize;
	// Thumbnail managment for tile items
	TSharedPtr<FTileThumbnailCollection>	ThumbnailCollection;
};

//----------------------------------------------------------------
//
//
//----------------------------------------------------------------
SWorldComposition::SWorldComposition()
{
}

SWorldComposition::~SWorldComposition()
{
	FWorldBrowserModule& WorldBrowserModule = FModuleManager::GetModuleChecked<FWorldBrowserModule>("WorldBrowser");
	WorldBrowserModule.OnBrowseWorld.RemoveAll(this);
	
	OnBrowseWorld(nullptr);
}

void SWorldComposition::Construct(const FArguments& InArgs)
{
	FWorldBrowserModule& WorldBrowserModule = FModuleManager::GetModuleChecked<FWorldBrowserModule>("WorldBrowser");
	WorldBrowserModule.OnBrowseWorld.AddSP(this, &SWorldComposition::OnBrowseWorld);

	ChildSlot
	[
		SAssignNew(ContentParent, SBorder)
		.BorderImage(FAppStyle::GetBrush(TEXT("ToolPanel.GroupBorder")))
	];
	
	OnBrowseWorld(InArgs._InWorld);
}

void SWorldComposition::OnBrowseWorld(UWorld* InWorld)
{
	// Remove old world bindings
	ContentParent->SetContent(SNullWidget::NullWidget);
	LayersListWrapBox = nullptr;
	NewLayerButton = nullptr;
	NewLayerMenu.Reset();
	GridView = nullptr;
	TileWorldModel = nullptr;
			
	// Bind to a new world model in case it's a world composition
	if (InWorld && InWorld->WorldComposition)
	{
		// Get the shared world model for this world object
		FWorldBrowserModule& WorldBrowserModule = FModuleManager::GetModuleChecked<FWorldBrowserModule>("WorldBrowser");
		auto SharedWorldModel = WorldBrowserModule.SharedWorldModel(InWorld);
		
		// double check we have a tile world
		if (SharedWorldModel->IsTileWorld())
		{
			TileWorldModel = StaticCastSharedPtr<FWorldTileCollectionModel>(SharedWorldModel);
			ContentParent->SetContent(ConstructContentWidget());
			PopulateLayersList();
		}
	}
}

TSharedRef<SWidget> SWorldComposition::ConstructContentWidget()
{
	return 	
		SNew(SVerticalBox)

		// Layers list
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SAssignNew(LayersListWrapBox, SWrapBox)
			.UseAllottedSize(true)
		]
				
		+SVerticalBox::Slot()
		.FillHeight(1)
		[
			SNew( SOverlay )

			// Grid view
			+SOverlay::Slot()
			[
				SAssignNew(GridView, SWorldCompositionGrid)
					.InWorldModel(TileWorldModel)
			]

			// Grid view top status bar
			+SOverlay::Slot()
			.VAlign(VAlign_Top)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush(TEXT("Graph.TitleBackground")))
				[
					SNew(SVerticalBox)

					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SHorizontalBox)
							
						// Current world view scale
						+SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNullWidget::NullWidget
						]
						+SHorizontalBox::Slot()
						.Padding(5,0,0,0)
						[
							SNullWidget::NullWidget
						]
							
						// World origin position
						+SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(SImage)
							.Image(FAppStyle::GetBrush( "WorldBrowser.WorldOrigin" ))
						]
						+SHorizontalBox::Slot()
						.Padding(5,0,0,0)
						[
							SNew(STextBlock)
							.TextStyle( FAppStyle::Get(), "WorldBrowser.StatusBarText" )
							.Text(this, &SWorldComposition::GetCurrentOriginText)
						]

						// Current level
						+SHorizontalBox::Slot()
						.HAlign(HAlign_Right)
						.Padding(0,0,5,0)
						[
							SNew(STextBlock)
							.TextStyle( FAppStyle::Get(), "WorldBrowser.StatusBarText" )
							.Text(this, &SWorldComposition::GetCurrentLevelText)
						]											
					]
				]
			]
			// Grid view bottom status bar
			+SOverlay::Slot()
			.VAlign(VAlign_Bottom)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush(TEXT("Graph.TitleBackground")))
				[
					SNew(SVerticalBox)

					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SHorizontalBox)

						// Mouse location
						+SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(SImage)
							.Image(FAppStyle::GetBrush( "WorldBrowser.MouseLocation" ))
						]
						+SHorizontalBox::Slot()
						.Padding(5,0,0,0)
						[
							SNew(STextBlock)
							.TextStyle( FAppStyle::Get(), "WorldBrowser.StatusBarText" )
							.Text(this, &SWorldComposition::GetMouseLocationText)
						]

						// Selection marquee rectangle size
						+SHorizontalBox::Slot()
						.AutoWidth()
						[
							SNew(SImage)
							.Image(FAppStyle::GetBrush( "WorldBrowser.MarqueeRectSize" ))
						]
						+SHorizontalBox::Slot()
						.Padding(5,0,0,0)
						[
							SNew(STextBlock)
							.TextStyle( FAppStyle::Get(), "WorldBrowser.StatusBarText" )
							.Text(this, &SWorldComposition::GetMarqueeSelectionSizeText)
						]

						// World size
						+SHorizontalBox::Slot()
						.HAlign(HAlign_Right)
						[
							SNew(SHorizontalBox)

							+SHorizontalBox::Slot()
							.AutoWidth()
							[
								SNew(SImage)
								.Image(FAppStyle::GetBrush( "WorldBrowser.WorldSize" ))
							]

							+SHorizontalBox::Slot()
							.Padding(5,0,5,0)
							[
								SNew(STextBlock)
								.TextStyle( FAppStyle::Get(), "WorldBrowser.StatusBarText" )
								.Text(this, &SWorldComposition::GetWorldSizeText)
							]
						]											
					]
				]
			]

			// Top-right corner text indicating that simulation is active
			+SOverlay::Slot()
			.Padding(20)
			.VAlign(VAlign_Top)
			.HAlign(HAlign_Right)
			[
				SNew(STextBlock)
				.Visibility(this, &SWorldComposition::IsSimulationVisible)
				.TextStyle( FAppStyle::Get(), "Graph.SimulatingText" )
				.Text(LOCTEXT("SimulatingNotification", "SIMULATING"))
			]
		];
}

void SWorldComposition::PopulateLayersList()
{
	TArray<FWorldTileLayer>& AllLayers = TileWorldModel->GetLayers();
	
	LayersListWrapBox->ClearChildren();
	for (auto WorldLayer : AllLayers)
	{
		LayersListWrapBox->AddSlot()
		.Padding(1,1,0,0)
		[
			SNew(SWorldLayerButton)
				.InWorldModel(TileWorldModel)
				.WorldLayer(WorldLayer)
		];
	}

	// Add new layer button
	LayersListWrapBox->AddSlot()
	.Padding(1,1,0,0)
	[
		SAssignNew(NewLayerButton, SButton)
		.OnClicked(this, &SWorldComposition::NewLayer_Clicked)
		.ButtonColorAndOpacity(FLinearColor(0.2f, 0.2f, 0.2f, 0.2f))
		.Content()
		[
			SNew(SImage)
			.Image(FAppStyle::GetBrush("WorldBrowser.AddLayer"))
		]
	];
}

FReply SWorldComposition::NewLayer_Clicked()
{
	if (TileWorldModel->IsReadOnly())
	{
		return FReply::Handled();
	}
	
	TSharedRef<SNewWorldLayerPopup> CreateLayerWidget = 
		SNew(SNewWorldLayerPopup)
		.OnCreateLayer(this, &SWorldComposition::CreateNewLayer)
		.DefaultName(LOCTEXT("Layer_DefaultName", "MyLayer").ToString())
		.InWorldModel(TileWorldModel);

	NewLayerMenu = FSlateApplication::Get().PushMenu(
		this->AsShared(),
		FWidgetPath(),
		CreateLayerWidget,
		FSlateApplication::Get().GetCursorPos(),
		FPopupTransitionEffect( FPopupTransitionEffect::TypeInPopup )
		);

	return FReply::Handled();
}

FReply SWorldComposition::CreateNewLayer(const FWorldTileLayer& NewLayer)
{
	TileWorldModel->AddManagedLayer(NewLayer);
	PopulateLayersList();
	
	if (NewLayerMenu.IsValid())
	{
		NewLayerMenu.Pin()->Dismiss();
	}
		
	return FReply::Handled();
}

FText SWorldComposition::GetZoomText() const
{
	return GridView->GetZoomText();
}

FText SWorldComposition::GetCurrentOriginText() const
{
	UWorld* CurrentWorld = (TileWorldModel->IsSimulating() ? TileWorldModel->GetSimulationWorld() : TileWorldModel->GetWorld());
	return FText::Format(LOCTEXT("PositionXYFmt", "{0}, {1}"), FText::AsNumber(CurrentWorld->OriginLocation.X), FText::AsNumber(CurrentWorld->OriginLocation.Y));
}

FText SWorldComposition::GetCurrentLevelText() const
{
	UWorld* CurrentWorld = (TileWorldModel->IsSimulating() ? TileWorldModel->GetSimulationWorld() : TileWorldModel->GetWorld());

	if (CurrentWorld->GetCurrentLevel())
	{
		UPackage* Package = CurrentWorld->GetCurrentLevel()->GetOutermost();
		return FText::FromString(FPackageName::GetShortName(Package->GetName()));
	}
	
	return LOCTEXT("None", "None");
}

FText SWorldComposition::GetMouseLocationText() const
{
	FVector2D MouseLocation = GridView->GetMouseWorldLocation();
	return FText::Format(LOCTEXT("PositionXYFmt", "{0}, {1}"), FText::AsNumber(FMath::RoundToInt(MouseLocation.X)), FText::AsNumber(FMath::RoundToInt(MouseLocation.Y)));
}

FText SWorldComposition::GetMarqueeSelectionSizeText() const
{
	FVector2D MarqueeSize = GridView->GetMarqueeWorldSize();
	
	if (MarqueeSize.X > 0 && 
		MarqueeSize.Y > 0)
	{
		return FText::Format(LOCTEXT("SizeXYFmt", "{0} x {1}"), FText::AsNumber(FMath::RoundToInt(MarqueeSize.X)), FText::AsNumber(FMath::RoundToInt(MarqueeSize.Y)));
	}
	else
	{
		return FText::GetEmpty();
	}
}

FText SWorldComposition::GetWorldSizeText() const
{
	FIntPoint WorldSize = TileWorldModel->GetWorldSize();
	
	if (WorldSize.X > 0 && 
		WorldSize.Y > 0)
	{
		return FText::Format(LOCTEXT("SizeXYFmt", "{0} x {1}"), FText::AsNumber(WorldSize.X), FText::AsNumber(WorldSize.Y));
	}
	else
	{
		return FText::GetEmpty();
	}
}

EVisibility SWorldComposition::IsSimulationVisible() const
{
	return (TileWorldModel->IsSimulating() ? EVisibility::Visible : EVisibility::Hidden);
}

#undef LOCTEXT_NAMESPACE
