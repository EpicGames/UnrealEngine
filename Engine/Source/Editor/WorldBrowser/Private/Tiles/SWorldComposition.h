// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Framework/Application/IMenu.h"
#include "Tiles/WorldTileCollectionModel.h"

class SButton;
class SWorldCompositionGrid;
class SWrapBox;

class SWorldComposition 
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SWorldComposition)
		:_InWorld(nullptr)
		{}
		SLATE_ARGUMENT(UWorld*, InWorld)
	SLATE_END_ARGS()

	SWorldComposition();
	~SWorldComposition();

	void Construct(const FArguments& InArgs);

private:
	/**  */
	void OnBrowseWorld(UWorld* InWorld);

	/**  */
	TSharedRef<SWidget> ConstructContentWidget();

	/** Populate current FWorldTileLayer list to UI */
	void PopulateLayersList();

	/** Creates a popup window with New layer parameters */
	FReply NewLayer_Clicked();
	
	/** Creates a new managed layer */
	FReply CreateNewLayer(const FWorldTileLayer& NewLayer);

	/** Top status bar details */
	FText GetZoomText() const;
	FText GetCurrentOriginText() const;
	FText GetCurrentLevelText() const;

	/** Bottom status bar details */
	FText GetMouseLocationText() const;
	FText GetMarqueeSelectionSizeText() const;
	FText GetWorldSizeText() const;

	/** @return whether SIMULATION sign should be visible */
	EVisibility IsSimulationVisible() const;
	
private:
	TSharedPtr<class FWorldTileCollectionModel> TileWorldModel;
	
	TSharedPtr<SBorder>							ContentParent;
	TSharedPtr<SWrapBox>						LayersListWrapBox;
	TSharedPtr<SButton>							NewLayerButton;
	TWeakPtr<class IMenu>						NewLayerMenu;
	TSharedPtr<class SWorldCompositionGrid>		GridView;
};
