// Copyright Epic Games, Inc. All Rights Reserved.
#include "Tiles/SWorldLayers.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Styling/AppStyle.h"
#include "Tiles/WorldTileCollectionModel.h"
#include "SlateOptMacros.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"

#include "Widgets/Input/SNumericEntryBox.h"

#define LOCTEXT_NAMESPACE "WorldBrowser"

//----------------------------------------------------------------
//
//
//----------------------------------------------------------------
BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SNewWorldLayerPopup::Construct(const FArguments& InArgs)
{
	OnCreateLayer	= InArgs._OnCreateLayer;
	LayerData.Name	= InArgs._DefaultName;

	// store set of currently existing layer names
	{
		const auto& AllLayersList = InArgs._InWorldModel->GetLayers();
		for (const auto& WorldLayer : AllLayersList)
		{
			ExistingLayerNames.Add(WorldLayer.Name);
		}
	}
	
	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("Menu.Background"))
		.Padding(10)
		[
			SNew(SVerticalBox)

			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2,2,0,0)
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("Layer_Name", "Name:"))
				]

				+SHorizontalBox::Slot()
				.Padding(4,0,0,0)
				[
					SNew(SEditableTextBox)
					.Text(this, &SNewWorldLayerPopup::GetLayerName)
					.SelectAllTextWhenFocused(true)
					.OnTextChanged(this, &SNewWorldLayerPopup::SetLayerName)
					.OnTextCommitted(this, &SNewWorldLayerPopup::OnNameCommitted)
					.ClearKeyboardFocusOnCommit(false)
				]

			]

			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2,2,0,0)
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SCheckBox)
					.IsChecked(this, &SNewWorldLayerPopup::GetDistanceStreamingState)
					.OnCheckStateChanged(this, &SNewWorldLayerPopup::OnDistanceStreamingStateChanged)
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SNumericEntryBox<int32>)
					.IsEnabled(this, &SNewWorldLayerPopup::IsDistanceStreamingEnabled)
					.Value(this, &SNewWorldLayerPopup::GetStreamingDistance)
					.MinValue(1)
					.MaxValue(TNumericLimits<int32>::Max())
					.OnValueChanged(this, &SNewWorldLayerPopup::SetStreamingDistance)
					.OnValueCommitted(this, &SNewWorldLayerPopup::OnDistanceCommitted)
					.LabelPadding(0)
					.Label()
					[
						SNumericEntryBox<int32>::BuildLabel(
							LOCTEXT("LayerStreamingDistance", "Streaming distance"), 
							FLinearColor::White, SNumericEntryBox<int32>::RedLabelBackgroundColor
							)

					]
				]
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2,2,0,0)
			[
				SNew(SButton)
				.OnClicked(this, &SNewWorldLayerPopup::OnClickedCreate)
				.IsEnabled(this, &SNewWorldLayerPopup::CanCreateLayer)
				.Text(LOCTEXT("Layer_Create", "Create"))
			]

		]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

FReply SNewWorldLayerPopup::OnClickedCreate()
{
	return TryCreateLayer();
}

void SNewWorldLayerPopup::OnNameCommitted(const FText& InText, ETextCommit::Type CommitType)
{
	if (CommitType == ETextCommit::OnEnter)
	{
		TryCreateLayer();
	}
}

void SNewWorldLayerPopup::OnDistanceCommitted(int32 InValue, ETextCommit::Type CommitType)
{
	if (CommitType == ETextCommit::OnEnter)
	{
		TryCreateLayer();
	}
}

FReply SNewWorldLayerPopup::TryCreateLayer()
{
	if (CanCreateLayer())
	{
		return OnCreateLayer.Execute(LayerData);
	}
	
	// Return an unhandled reply if the layer should not be created
	return FReply::Unhandled();
}

bool SNewWorldLayerPopup::CanCreateLayer() const
{
	const bool bValidStreamingDistance = LayerData.DistanceStreamingEnabled ? LayerData.StreamingDistance > 0 : true;

	return (bValidStreamingDistance &&
			LayerData.Name.Len() > 0 && 
			!ExistingLayerNames.Contains(LayerData.Name) && 
			OnCreateLayer.IsBound());
}

/** A class for check boxes in the layer list. 
 *	If you double click a layer checkbox, you will enable it and disable all others 
 *	If you Ctrl+Click a layer checkbox, you will add/remove it from selection list
 */
class SLayerCheckBox : public SCheckBox
{
public:
	void SetOnLayerDoubleClicked(const FOnClicked& NewLayerDoubleClicked)
	{
		OnLayerDoubleClicked = NewLayerDoubleClicked;
	}

	void SetOnLayerCtrlClicked(const FOnClicked& NewLayerCtrlClicked)
	{
		OnLayerCtrlClicked = NewLayerCtrlClicked;
	}
	
	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override
	{
		if (OnLayerDoubleClicked.IsBound())
		{
			return OnLayerDoubleClicked.Execute();
		}
		else
		{
			return SCheckBox::OnMouseButtonDoubleClick(InMyGeometry, InMouseEvent);
		}
	}

	virtual FReply OnMouseButtonUp(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override
	{
		if (!InMouseEvent.IsControlDown())
		{
			return SCheckBox::OnMouseButtonUp(InMyGeometry, InMouseEvent);
		}
		
		if (InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			bIsPressed = false;

			if (IsHovered() && HasMouseCapture())
			{
				if (OnLayerCtrlClicked.IsBound())
				{
					return OnLayerCtrlClicked.Execute();
				}
			}
		}

		return FReply::Handled().ReleaseMouseCapture();
	}


private:
	FOnClicked OnLayerDoubleClicked;
	FOnClicked OnLayerCtrlClicked;
};

//----------------------------------------------------------------
//
//
//----------------------------------------------------------------
SWorldLayerButton::~SWorldLayerButton()
{
}

void SWorldLayerButton::Construct(const FArguments& InArgs)
{
	WorldModel = InArgs._InWorldModel;
	WorldLayer = InArgs._WorldLayer;

	TSharedPtr<SLayerCheckBox> CheckBox;
	
	ChildSlot
		[
			SNew(SBorder)
				.BorderBackgroundColor(FLinearColor(0.2f, 0.2f, 0.2f, 0.2f))
				.BorderImage(FAppStyle::GetBrush("ContentBrowser.FilterButtonBorder"))
				[
					SAssignNew(CheckBox, SLayerCheckBox)
						.Style(FAppStyle::Get(), "ToggleButtonCheckbox")
						.OnCheckStateChanged(this, &SWorldLayerButton::OnCheckStateChanged)
						.IsChecked(this, &SWorldLayerButton::IsChecked)
						.OnGetMenuContent(this, &SWorldLayerButton::GetRightClickMenu)
						.ToolTipText(this, &SWorldLayerButton::GetToolTipText)
						.Padding(3)
						[
							SNew(STextBlock)
								.ShadowOffset(FVector2D(1.f, 1.f))
								.Text(FText::FromString(WorldLayer.Name))
						]
				]
		];

	CheckBox->SetOnLayerCtrlClicked(FOnClicked::CreateSP(this, &SWorldLayerButton::OnCtrlClicked));
	CheckBox->SetOnLayerDoubleClicked(FOnClicked::CreateSP(this, &SWorldLayerButton::OnDoubleClicked));
}

void SWorldLayerButton::OnCheckStateChanged(ECheckBoxState NewState)
{
	if (NewState == ECheckBoxState::Checked)
	{
		WorldModel->SetSelectedLayer(WorldLayer);
	}
	else
	{
		WorldModel->SetSelectedLayers(TArray<FWorldTileLayer>());
	}
}

ECheckBoxState SWorldLayerButton::IsChecked() const
{
	return WorldModel->IsLayerSelected(WorldLayer) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

/** Handler for when the filter checkbox is double clicked */
FReply SWorldLayerButton::OnDoubleClicked()
{
	return FReply::Handled().ReleaseMouseCapture();
}

FReply SWorldLayerButton::OnCtrlClicked()
{
	WorldModel->ToggleLayerSelection(WorldLayer);
	return FReply::Handled().ReleaseMouseCapture();
}

TSharedRef<SWidget> SWorldLayerButton::GetRightClickMenu()
{
	return SNullWidget::NullWidget;
}

FText SWorldLayerButton::GetToolTipText() const
{
	if (WorldLayer.DistanceStreamingEnabled)
	{
		return FText::Format(LOCTEXT("Layer_Distance_Tooltip", "Streaming Distance: {0}"), FText::AsNumber(WorldLayer.StreamingDistance));
	}
	else
	{
		return FText(LOCTEXT("Layer_DisabledDistance_Tooltip", "Distance Streaming Disabled"));
	}
}

#undef LOCTEXT_NAMESPACE
