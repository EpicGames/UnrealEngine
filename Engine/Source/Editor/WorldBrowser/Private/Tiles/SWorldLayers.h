// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Misc/WorldCompositionUtility.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "Styling/SlateTypes.h"
#include "Widgets/Layout/SBorder.h"

class FWorldTileCollectionModel;

//----------------------------------------------------------------
//
//
//----------------------------------------------------------------
class SWorldLayerButton 
	: public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SWorldLayerButton)
	{}
		/** Data for the asset this item represents */
		SLATE_ARGUMENT(FWorldTileLayer, WorldLayer)
		SLATE_ARGUMENT(TSharedPtr<FWorldTileCollectionModel>, InWorldModel)
	SLATE_END_ARGS()

	~SWorldLayerButton();
	void Construct(const FArguments& InArgs);
	void OnCheckStateChanged(ECheckBoxState NewState);
	ECheckBoxState IsChecked() const;
	FReply OnDoubleClicked();
	FReply OnCtrlClicked();
	TSharedRef<SWidget> GetRightClickMenu();
	FText GetToolTipText() const;
			
private:
	/** The data for this item */
	TSharedPtr<FWorldTileCollectionModel>	WorldModel;
	FWorldTileLayer							WorldLayer;
};


class SNewWorldLayerPopup 
	: public SBorder
{
public:
	DECLARE_DELEGATE_RetVal_OneParam(FReply, FOnCreateLayer, const FWorldTileLayer&)
	
	SLATE_BEGIN_ARGS(SNewWorldLayerPopup)
	{}
	SLATE_EVENT(FOnCreateLayer, OnCreateLayer)
	SLATE_ARGUMENT(FString, DefaultName)
	SLATE_ARGUMENT(TSharedPtr<FWorldTileCollectionModel>, InWorldModel)
	SLATE_END_ARGS()
		
	void Construct(const FArguments& InArgs);

	TOptional<int32> GetStreamingDistance() const
	{
		return LayerData.StreamingDistance;
	}
	
	ECheckBoxState GetDistanceStreamingState() const
	{
		return IsDistanceStreamingEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	bool IsDistanceStreamingEnabled() const
	{
		return LayerData.DistanceStreamingEnabled;
	}
	
	void OnDistanceStreamingStateChanged(ECheckBoxState NewState)
	{
		SetDistanceStreamingState(NewState == ECheckBoxState::Checked);
	}

	FText GetLayerName() const
	{
		return FText::FromString(LayerData.Name);
	}
	
private:
	FReply OnClickedCreate();

	void OnNameCommitted(const FText& InText, ETextCommit::Type CommitType);
	void OnDistanceCommitted(int32 InValue, ETextCommit::Type CommitType);

	/** Try to create a new streaming layer */
	FReply TryCreateLayer();

	bool CanCreateLayer() const;

	void SetLayerName(const FText& InText)
	{
		LayerData.Name = InText.ToString();
	}

	void SetStreamingDistance(int32 InValue)
	{
		LayerData.StreamingDistance = InValue;
	}

	void SetDistanceStreamingState(bool bIsEnabled)
	{
		LayerData.DistanceStreamingEnabled = bIsEnabled;
	}

private:
	/** The delegate to execute when the create button is clicked */
	FOnCreateLayer							OnCreateLayer;
	FWorldTileLayer							LayerData;
	TSet<FString>							ExistingLayerNames;
};
