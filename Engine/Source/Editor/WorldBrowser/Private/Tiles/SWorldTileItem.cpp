// Copyright Epic Games, Inc. All Rights Reserved.
#include "Tiles/SWorldTileItem.h"
#include "Rendering/DrawElements.h"
#include "Widgets/SBoxPanel.h"
#include "SlateOptMacros.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/SToolTip.h"
#include "LevelModel.h"

#include "Tiles/WorldTileCollectionModel.h"
#include "Tiles/WorldTileDetails.h"
#include "Tiles/WorldTileThumbnails.h"

#define LOCTEXT_NAMESPACE "WorldBrowser"

//----------------------------------------------------------------
//
//
//----------------------------------------------------------------
int32 SWorldTileImage::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const FSlateBrush* ImageBrush = GetImageAttribute().Get();

	if ((ImageBrush != nullptr) && (ImageBrush->DrawAs != ESlateBrushDrawType::NoDrawType))
	{
		const bool bIsEnabled = EditableTile.Get() && ShouldBeEnabled(bParentEnabled);
		const ESlateDrawEffect DrawEffects = bIsEnabled ? ESlateDrawEffect::None : ESlateDrawEffect::DisabledEffect;
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			LayerId,
			AllottedGeometry.ToPaintGeometry(),
			ImageBrush,
			DrawEffects | ESlateDrawEffect::IgnoreTextureAlpha, 
			FColor::White);
	}
	return LayerId;
}

//----------------------------------------------------------------
//
//
//----------------------------------------------------------------
SWorldTileItem::SWorldTileItem()
 : bAffectedByMarquee(false)
 , bNeedRefresh(false)
 , bIsDragging(false)
{
}

SWorldTileItem::~SWorldTileItem()
{
	ThumbnailImageWidget->SetImage(nullptr);
	ThumbnailCollection->UnregisterTile(*TileModel.Get());
	TileModel->ChangedEvent.RemoveAll(this);
}

void SWorldTileItem::Construct(const FArguments& InArgs)
{
	WorldModel = InArgs._InWorldModel;
	TileModel = InArgs._InItemModel;
	ThumbnailCollection = InArgs._InThumbnailCollection;
	
	TileModel->ChangedEvent.AddSP(this, &SWorldTileItem::RequestRefresh);
	
	GetOrAddSlot( ENodeZone::Center )
	[
		SAssignNew(ThumbnailImageWidget, SWorldTileImage)
		.EditableTile(this, &SWorldTileItem::IsItemEnabled)
	];

	ThumbnailCollection->RegisterTile(*TileModel.Get());
	const FSlateBrush* TileBrush = ThumbnailCollection->GetTileBrush(*TileModel.Get());
	ThumbnailImageWidget->SetImage(TileBrush);

	SetToolTip(CreateToolTipWidget());

	bNeedRefresh = true;
}

void SWorldTileItem::RequestRefresh()
{
	bNeedRefresh = true;
}

UObject* SWorldTileItem::GetObjectBeingDisplayed() const
{
	return TileModel->GetNodeObject();
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef<SToolTip> SWorldTileItem::CreateToolTipWidget()
{
	TSharedPtr<SToolTip> TooltipWidget;
	
	SAssignNew(TooltipWidget, SToolTip)
	.TextMargin(2)
	.BorderImage(FAppStyle::GetBrush("ContentBrowser.TileViewTooltip.NonContentBorder"))
	[
		SNew(SVerticalBox)

		// Level name section
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0,0,0,4)
		[
			SNew(SBorder)
			.Padding(6)
			.BorderImage(FAppStyle::GetBrush("ContentBrowser.TileViewTooltip.ContentBorder"))
			[
				SNew(SVerticalBox)
					
				+SVerticalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoHeight()
				[
					SNew(SHorizontalBox)

					// Level name
					+SHorizontalBox::Slot()
					.Padding(6,0,0,0)
					.HAlign(HAlign_Left)
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text(this, &SWorldTileItem::GetLevelNameText) 
						.Font(FAppStyle::GetFontStyle("ContentBrowser.TileViewTooltip.NameFont"))
					]
				]
			]
		]
			
		// Tile info section
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.Padding(6)
			.BorderImage(FAppStyle::GetBrush("ContentBrowser.TileViewTooltip.ContentBorder"))
			[
				SNew(SUniformGridPanel)

				// Tile position
				+SUniformGridPanel::Slot(0, 0)
				.HAlign(HAlign_Left)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("Item_OriginOffset", "Position:")) 
				]
					
				+SUniformGridPanel::Slot(1, 0)
				.HAlign(HAlign_Right)
				[
					SNew(STextBlock)
					.Text(this, &SWorldTileItem::GetPositionText) 
				]

				// Tile bounds
				+SUniformGridPanel::Slot(0, 1)
				.HAlign(HAlign_Left)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("Item_BoundsExtent", "Extent:")) 
				]
					
				+SUniformGridPanel::Slot(1, 1)
				.HAlign(HAlign_Right)
				[
					SNew(STextBlock)
					.Text(this, &SWorldTileItem::GetBoundsExtentText) 
				]

				// Layer name
				+SUniformGridPanel::Slot(0, 2)
				.HAlign(HAlign_Left)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("Item_Name", "Layer Name:")) 
				]
					
				+SUniformGridPanel::Slot(1, 2)
				.HAlign(HAlign_Right)
				[
					SNew(STextBlock)
					.Text(this, &SWorldTileItem::GetLevelLayerNameText) 
				]

				// Streaming distance
				+SUniformGridPanel::Slot(0, 3)
				.HAlign(HAlign_Left)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("Item_Distance", "Streaming Distance:")) 
				]
					
				+SUniformGridPanel::Slot(1, 3)
				.HAlign(HAlign_Right)
				[
					SNew(STextBlock)
					.Text(this, &SWorldTileItem::GetLevelLayerDistanceText) 
				]
			]
		]
	];

	return TooltipWidget.ToSharedRef();
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

FVector2f SWorldTileItem::GetPosition2f() const
{
	return UE::Slate::CastToVector2f(TileModel->GetLevelPosition2D());
}

TSharedPtr<FLevelModel> SWorldTileItem::GetLevelModel() const
{
	return StaticCastSharedPtr<FLevelModel>(TileModel);
}

const FSlateBrush* SWorldTileItem::GetShadowBrush(bool bSelected) const
{
	return bSelected ? FAppStyle::GetBrush(TEXT("Graph.CompactNode.ShadowSelected")) : FAppStyle::GetBrush(TEXT("Graph.Node.Shadow"));
}

void SWorldTileItem::GetDiffHighlightBrushes(const FSlateBrush*& BackgroundOut, const FSlateBrush*& ForegroundOut) const
{
	BackgroundOut = FAppStyle::GetBrush(TEXT("Graph.CompactNode.DiffHighlight"));
	ForegroundOut = FAppStyle::GetBrush(TEXT("Graph.CompactNode.DiffHighlightShading"));
}

FOptionalSize SWorldTileItem::GetItemWidth() const
{
	return FOptionalSize(TileModel->GetLevelSize2D().X);
}

FOptionalSize SWorldTileItem::GetItemHeight() const
{
	return FOptionalSize(TileModel->GetLevelSize2D().Y);
}

FSlateRect SWorldTileItem::GetItemRect() const
{
	FVector2f LevelSize = UE::Slate::CastToVector2f(TileModel->GetLevelSize2D());
	FVector2f LevelPos = GetPosition2f();
	return FSlateRect(LevelPos, LevelPos + LevelSize);
}

TSharedPtr<IToolTip> SWorldTileItem::GetToolTip()
{
	// Hide tooltip in case item is being dragged now
	if (TileModel->GetLevelTranslationDelta().SizeSquared() > FMath::Square(KINDA_SMALL_NUMBER))
	{
		return NULL;
	}
	
	return SNodePanel::SNode::GetToolTip();
}

FVector2f SWorldTileItem::GetDesiredSizeForMarquee2f() const
{
	// we don't want to select items in non visible layers
	if (!WorldModel->PassesAllFilters(*TileModel))
	{
		return FVector2f::ZeroVector;
	}

	return SNodePanel::SNode::GetDesiredSizeForMarquee2f();
}

FVector2D SWorldTileItem::ComputeDesiredSize( float ) const
{
	return TileModel->GetLevelSize2D();
}

int32 SWorldTileItem::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& ClippingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	const bool bIsVisible = FSlateRect::DoRectanglesIntersect(AllottedGeometry.GetLayoutBoundingRect(), ClippingRect);
	
	if (bIsVisible)
	{
		// Redraw thumbnail image if requested
		if (bNeedRefresh && !ThumbnailCollection->IsOnCooldown())
		{
			bNeedRefresh = false;
			const FSlateBrush* TileBrush = ThumbnailCollection->UpdateTileThumbnail(*TileModel.Get());
			ThumbnailImageWidget->SetImage(TileBrush);
		}
		
		LayerId = SNodePanel::SNode::OnPaint(Args, AllottedGeometry, ClippingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
		
		const bool bSelected = (IsItemSelected() || bAffectedByMarquee);
		const int32* PreviewLODIndex = WorldModel->GetPreviewStreamingLevels().Find(TileModel->TileDetails->PackageName);
		const bool bHighlighted = (PreviewLODIndex != nullptr);

		// Draw the node's selection/highlight.
		if (bSelected || bHighlighted)
		{
			// Calculate selection box paint geometry 
			const FVector2D InflateAmount = FVector2D(4, 4);
			const float Scale = 0.5f; // Scale down image of the borders to make them thinner 
			FSlateLayoutTransform LayoutTransform(Scale, AllottedGeometry.GetAccumulatedLayoutTransform().GetTranslation() - InflateAmount);
			FSlateRenderTransform SlateRenderTransform(Scale, AllottedGeometry.GetAccumulatedRenderTransform().GetTranslation() - InflateAmount);
			FPaintGeometry SelectionGeometry(LayoutTransform, SlateRenderTransform, (AllottedGeometry.GetLocalSize()*AllottedGeometry.Scale + InflateAmount*2)/Scale, !SlateRenderTransform.IsIdentity());
			FLinearColor HighlightColor = FLinearColor::White;
			if (PreviewLODIndex)
			{
				// Highlight LOD tiles in different color to normal tiles
				HighlightColor = (*PreviewLODIndex == INDEX_NONE) ? FLinearColor::Green : FLinearColor(0.3f,1.0f,0.3f);
			}
			
			FSlateDrawElement::MakeBox(
				OutDrawElements,
				LayerId + 1,
				SelectionGeometry,
				GetShadowBrush(bSelected || bHighlighted),
				ESlateDrawEffect::None,
				HighlightColor
				);
		}
	}
	
	return LayerId;
}

FReply SWorldTileItem::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	TileModel->MakeLevelCurrent();
	return FReply::Handled();
}

FText SWorldTileItem::GetLevelNameText() const
{
	return FText::FromString(TileModel->GetDisplayName());
}

FText SWorldTileItem::GetPositionText() const
{
	FIntVector Position = TileModel->GetRelativeLevelPosition();
	bool bLocked = WorldModel->IsLockTilesLocationEnabled();
	
	FTextFormat TextFormat;
	if (bLocked)
	{
		TextFormat = LOCTEXT("PositionXYZFmtLocked", "{0}, {1}, {2} (Locked)");
	}
	else
	{
		TextFormat = LOCTEXT("PositionXYZFmt", "{0}, {1}, {2}");
	}
		
	return FText::Format(TextFormat, FText::AsNumber(Position.X), FText::AsNumber(Position.Y), FText::AsNumber(Position.Z));
}

FText SWorldTileItem::GetBoundsExtentText() const
{
	FVector2D Size = TileModel->GetLevelSize2D();
	return FText::Format(LOCTEXT("PositionXYFmt", "{0}, {1}"), FText::AsNumber(FMath::RoundToInt(Size.X*0.5f)), FText::AsNumber(FMath::RoundToInt(Size.Y*0.5f)));
}

FText SWorldTileItem::GetLevelLayerNameText() const
{
	return FText::FromString(TileModel->TileDetails->Layer.Name);
}

FText SWorldTileItem::GetLevelLayerDistanceText() const
{
	if (TileModel->TileDetails->Layer.DistanceStreamingEnabled)
	{
		return FText::AsNumber(TileModel->TileDetails->Layer.StreamingDistance);
	}
	else
	{
		return FText(LOCTEXT("DistanceStreamingDisabled", "Distance Streaming Disabled"));
	}
}

bool SWorldTileItem::IsItemEditable() const
{
	return TileModel->IsEditable();
}

bool SWorldTileItem::IsItemSelected() const
{
	return TileModel->GetLevelSelectionFlag();
}

bool SWorldTileItem::IsItemEnabled() const
{
	if (WorldModel->IsSimulating())
	{
		return TileModel->IsVisibleInEditor();
	}
	else
	{
		return TileModel->IsEditable();
	}
}

#undef LOCTEXT_NAMESPACE
