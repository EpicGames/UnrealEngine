// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Layout/SlateRect.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Types/SlateStructs.h"
#include "Widgets/Images/SImage.h"
#include "SNodePanel.h"

class FLevelModel;
class FSlateWindowElementList;
class FTileThumbnailCollection;
class FWorldTileCollectionModel;
class FWorldTileModel;
class IToolTip;
class SToolTip;

//----------------------------------------------------------------
//
//
//----------------------------------------------------------------
class SWorldTileImage : public SImage
{
	SLATE_BEGIN_ARGS(SWorldTileImage)
		: _EditableTile(false)
	{}
		SLATE_ATTRIBUTE(bool, EditableTile)
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs)
	{
		EditableTile = InArgs._EditableTile;
	}

	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;

private:
	TAttribute<bool> EditableTile;
};

//----------------------------------------------------------------
//
//
//----------------------------------------------------------------
class SWorldTileItem 
	: public SNodePanel::SNode
{
public:
	SLATE_BEGIN_ARGS(SWorldTileItem)
	{}
		/** The world data */
		SLATE_ARGUMENT(TSharedPtr<FWorldTileCollectionModel>, InWorldModel)
		/** Data for the asset this item represents */
		SLATE_ARGUMENT(TSharedPtr<FWorldTileModel>, InItemModel)
		/** Thumbnails management */
		SLATE_ARGUMENT(TSharedPtr<FTileThumbnailCollection>, InThumbnailCollection)
	SLATE_END_ARGS()

	SWorldTileItem();
	~SWorldTileItem();

	void Construct(const FArguments& InArgs);
		
	// SNodePanel::SNode interface start
	virtual FVector2f GetDesiredSizeForMarquee2f() const override;
	virtual UObject* GetObjectBeingDisplayed() const override;
	virtual FVector2f GetPosition2f() const override;
	virtual const FSlateBrush* GetShadowBrush(bool bSelected) const override;
	virtual void GetDiffHighlightBrushes(const FSlateBrush*& BackgroundOut, const FSlateBrush*& ForegroundOut) const override;
	// SNodePanel::SNode interface end
	
	/** @return Deferred item refresh */
	void RequestRefresh();

	/** @return LevelModel associated with this item */
	TSharedPtr<FLevelModel>	GetLevelModel() const;
	
	/** @return Item width in world units */
	FOptionalSize GetItemWidth() const;
	
	/** @return Item height in world units */
	FOptionalSize GetItemHeight() const;
	
	/** @return Rectangle in world units for this item as FSlateRect*/
	FSlateRect GetItemRect() const;
	
	/** @return Whether this item can be edited (loaded and not locked) */
	bool IsItemEditable() const;

	/** @return Whether this item is selected */
	bool IsItemSelected() const;
	
	/** @return Whether this item is enabled */
	bool IsItemEnabled() const;

private:
	// SWidget interface start
	virtual TSharedPtr<IToolTip> GetToolTip() override;
	virtual FVector2D ComputeDesiredSize(float) const override;
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	// SWidget interface end

	TSharedRef<SToolTip> CreateToolTipWidget();
	
	/** Tile tooltips fields */
	FText GetLevelNameText() const;
	FText GetPositionText() const;
	FText GetBoundsExtentText() const;
	FText GetLevelLayerNameText() const;
	FText GetLevelLayerDistanceText() const;
	
public:
	bool									bAffectedByMarquee;

private:
	/** The world data */
	TSharedPtr<FWorldTileCollectionModel>	WorldModel;
	/** The data for this item */
	TSharedPtr<FWorldTileModel>				TileModel;
	
	TSharedPtr<SWorldTileImage>				ThumbnailImageWidget;	
	TSharedPtr<FTileThumbnailCollection>	ThumbnailCollection;
	
	mutable bool							bNeedRefresh;
	bool									bIsDragging;
};

