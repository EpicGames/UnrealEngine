// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tiles/WorldTileCollectionModel.h"

#include "Misc/PackageName.h"
#include "Components/PrimitiveComponent.h"
#include "Misc/CoreDelegates.h"
#include "Misc/MessageDialog.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"
#include "Widgets/SWindow.h"
#include "MeshMerge/MeshProxySettings.h"
#include "Engine/StaticMesh.h"
#include "Engine/Light.h"
#include "Engine/StaticMeshActor.h"
#include "Editor.h"
#include "LevelEditorViewport.h"
#include "EditorModeManager.h"
#include "EditorDirectories.h"
#include "FileHelpers.h"
#include "LevelCollectionCommands.h"
#include "ScopedTransaction.h"

#include "Engine/WorldComposition.h"
#include "IDetailsView.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "MeshUtilities.h"

#include "StaticMeshAttributes.h"
#include "Materials/Material.h"
#include "MaterialUtilities.h"
#include "LandscapeLayerInfoObject.h"
#include "LandscapeEditorUtils.h"

#include "Tiles/WorldTileDetails.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Tiles/WorldTileDetailsCustomization.h"
#include "Tiles/STiledLandscapeImportDlg.h"
#include "Landscape.h"
#include "Misc/FeedbackContext.h"
#include "UObject/UObjectIterator.h"
#include "GameFramework/PlayerController.h"
#include "Components/HierarchicalInstancedStaticMeshComponent.h"
#include "InstancedFoliageActor.h"
#include "LandscapeMeshProxyActor.h"
#include "LandscapeMeshProxyComponent.h"
#include "LandscapeFileFormatInterface.h"
#include "LandscapeEditorModule.h"
#include "LandscapeUtils.h"
#include "IMeshReductionManagerModule.h"
#include "IMeshMergeUtilities.h"
#include "MeshMergeModule.h"
#include "PrimitiveSceneProxy.h"
#include "UObject/ObjectSaveContext.h"

#define LOCTEXT_NAMESPACE "WorldBrowser"

static const FName HeightmapLayerName = FName("__Heightmap__");


FWorldTileCollectionModel::FWorldTileCollectionModel()
	: FLevelCollectionModel()
	, PreviewLocation(0.0f,0.0f,0.0f)
	, bIsSavingLevel(false)
	, bMeshProxyAvailable(false)
{
}

FWorldTileCollectionModel::~FWorldTileCollectionModel()
{
	// There are still can be levels loading 
	FlushAsyncLoading();
	
	CurrentWorld = NULL;

	GEditor->UnregisterForUndo(this);
	FCoreDelegates::PreWorldOriginOffset.RemoveAll(this);
	FCoreDelegates::PostWorldOriginOffset.RemoveAll(this);
	FEditorDelegates::PreSaveWorldWithContext.RemoveAll(this);
	FEditorDelegates::PostSaveWorldWithContext.RemoveAll(this);
	FEditorDelegates::NewCurrentLevel.RemoveAll(this);
}

void FWorldTileCollectionModel::Initialize(UWorld* InWorld)
{
	// Uncategorized layer, always exist
	FWorldTileLayer Layer;
	ManagedLayers.Empty();
	ManagedLayers.Add(Layer);

	GEditor->RegisterForUndo(this);
	FCoreDelegates::PreWorldOriginOffset.AddSP(this, &FWorldTileCollectionModel::PreWorldOriginOffset);
	FCoreDelegates::PostWorldOriginOffset.AddSP(this, &FWorldTileCollectionModel::PostWorldOriginOffset);
	FEditorDelegates::PreSaveWorldWithContext.AddSP(this, &FWorldTileCollectionModel::OnPreSaveWorld);
	FEditorDelegates::PostSaveWorldWithContext.AddSP(this, &FWorldTileCollectionModel::OnPostSaveWorld);
	FEditorDelegates::NewCurrentLevel.AddSP(this, &FWorldTileCollectionModel::OnNewCurrentLevel);
	BindCommands();
			
	FLevelCollectionModel::Initialize(InWorld);
	
	// Check whehter Editor has support for generating mesh proxies	
	IMeshReductionModule& ReductionModule = FModuleManager::Get().LoadModuleChecked<IMeshReductionModule>("MeshReductionInterface");
	bMeshProxyAvailable = (ReductionModule.GetMeshMergingInterface() != nullptr);
}

void FWorldTileCollectionModel::Tick( float DeltaTime )
{
	if (!HasWorldRoot())
	{
		return;
	}

	FLevelCollectionModel::Tick(DeltaTime);
}

void FWorldTileCollectionModel::UnloadLevels(const FLevelModelList& InLevelList)
{
	if (IsReadOnly())
	{
		return;
	}
	
	// Check dirty levels
	bool bHasDirtyLevels = false;
	for (TSharedPtr<FLevelModel> LevelModel : InLevelList)
	{
		ULevel* Level = LevelModel->GetLevelObject();
		if (Level && Level->GetOutermost()->IsDirty())
		{
			bHasDirtyLevels = true;
			break;
		}
	}

	FLevelModelList LevelsToUnload = InLevelList;

	if (bHasDirtyLevels)
	{
		// Warn the user that they are about to remove dirty level(s) from the world
		const FText RemoveDirtyWarning = LOCTEXT("UnloadingDirtyLevelFromWorld", "You are about to unload dirty levels from the world and your changes to these levels will be lost (all children levels will be unloaded as well).  Proceed?");
		if (FMessageDialog::Open(EAppMsgType::YesNo, RemoveDirtyWarning) == EAppReturnType::No)
		{
			return;
		}

		// We need to unload all children of an dirty tiles,
		// to make sure that relative positions will be correct after parent tile information is discarded
		struct FHierarchyCollector : public FLevelModelVisitor
		{
			FLevelModelList& DirtyHierarchy;
			FHierarchyCollector(FLevelModelList& LevelToUnload) : DirtyHierarchy(LevelToUnload) {}

			virtual void Visit(FLevelModel& Item) override
			{
				DirtyHierarchy.AddUnique(Item.AsShared());
			}
		};

		FHierarchyCollector HierarchyCollector(LevelsToUnload);

		for (TSharedPtr<FLevelModel> LevelModel : InLevelList)
		{
			ULevel* Level = LevelModel->GetLevelObject();
			if (Level && Level->GetOutermost()->IsDirty())
			{
				LevelModel->Accept(HierarchyCollector);
			}
		}
	}

	// Unload
	FLevelCollectionModel::UnloadLevels(LevelsToUnload);
}

void FWorldTileCollectionModel::TranslateLevels(const FLevelModelList& InLevels, FVector2D InDelta, bool bSnapDelta)
{
	if (IsReadOnly() || InLevels.Num() == 0 || IsLockTilesLocationEnabled())
	{
		return;
	}
	
	// We want to translate only non-readonly levels
	FLevelModelList TilesToMove;
	for (auto It = InLevels.CreateConstIterator(); It; ++It)
	{
		if ((*It)->IsEditable())
		{
			TilesToMove.Add(*It);
		}
	}

	if (TilesToMove.Num() == 0)
	{
		return;
	}
	
	// Remove all descendants models from the list
	// We need to translate only top hierarchy models
	for (int32 TileIdx = TilesToMove.Num()-1; TileIdx >= 0; TileIdx--)
	{
		TSharedPtr<FLevelModel> TileModel = TilesToMove[TileIdx];
		for (int32 ParentIdx = 0; ParentIdx < TilesToMove.Num(); ++ParentIdx)
		{
			if (TileModel->HasAncestor(TilesToMove[ParentIdx]))
			{
				TilesToMove.RemoveAt(TileIdx);
				break;
			}
		}
	}

	// Calculate moving levels bounding box, prefer currently visible levels
	FBox LevelsBBox = GetVisibleLevelsBoundingBox(TilesToMove, true);
	if (!LevelsBBox.IsValid)
	{
		LevelsBBox = GetLevelsBoundingBox(TilesToMove, true);
	}

	const FScopedTransaction MoveLevelsTransaction(LOCTEXT("MoveLevelsTransaction", "Move Levels"));
	
	// Focus on levels destination bounding box, so the will stay visible after translation
	if (LevelsBBox.IsValid)
	{
		LevelsBBox = LevelsBBox.ShiftBy(FVector(InDelta, 0));
		Focus(LevelsBBox, EnsureEditable);
	}
		
	// Move levels
	for (auto It = TilesToMove.CreateIterator(); It; ++It)
	{
		TSharedPtr<FWorldTileModel> TileModel = StaticCastSharedPtr<FWorldTileModel>(*It);
		FIntVector NewPosition = TileModel->GetAbsoluteLevelPosition() + FIntVector(InDelta.X, InDelta.Y, 0);
		TileModel->SetLevelPosition(NewPosition);
	}

	// Unshelve levels which do fit to the current world bounds
	for (auto It = AllLevelsList.CreateIterator(); It; ++It)
	{
		TSharedPtr<FWorldTileModel> TileModel = StaticCastSharedPtr<FWorldTileModel>(*It);
		if (TileModel->ShouldBeVisible(EditableWorldArea()))
		{
			TileModel->Unshelve();
		}
	}
	
	RequestUpdateAllLevels();
}

TSharedPtr<WorldHierarchy::FWorldBrowserDragDropOp> FWorldTileCollectionModel::CreateDragDropOp() const
{
	return CreateDragDropOp(SelectedLevelsList);
}

TSharedPtr<WorldHierarchy::FWorldBrowserDragDropOp> FWorldTileCollectionModel::CreateDragDropOp(const FLevelModelList& InLevels) const
{
	TArray<TWeakObjectPtr<ULevel>>			LevelsToDrag;
	TArray<TWeakObjectPtr<ULevelStreaming>> StreamingLevelsToDrag;

	if (!IsReadOnly())
	{
		for (TSharedPtr<FLevelModel> LevelModel : InLevels)
		{
			check(AllLevelsList.Contains(LevelModel));
			ULevel* Level = LevelModel->GetLevelObject();
			if (Level)
			{
				LevelsToDrag.AddUnique(Level);
			}

			TSharedPtr<FWorldTileModel> Tile = StaticCastSharedPtr<FWorldTileModel>(LevelModel);
			if (Tile->IsLoaded())
			{
				StreamingLevelsToDrag.AddUnique(Tile->GetAssociatedStreamingLevel());
			}
			else
			{
				//
				int32 TileStreamingIdx = GetWorld()->WorldComposition->TilesStreaming.IndexOfByPredicate(
					ULevelStreaming::FPackageNameMatcher(LevelModel->GetLongPackageName())
				);

				if (GetWorld()->WorldComposition->TilesStreaming.IsValidIndex(TileStreamingIdx))
				{
					StreamingLevelsToDrag.AddUnique(GetWorld()->WorldComposition->TilesStreaming[TileStreamingIdx]);
				}
			}
		}
	}

	if (LevelsToDrag.Num())
	{
		TSharedPtr<WorldHierarchy::FWorldBrowserDragDropOp> Op = WorldHierarchy::FWorldBrowserDragDropOp::New(LevelsToDrag);
		Op->StreamingLevelsToDrop = StreamingLevelsToDrag;
		return Op;
	}

	if (StreamingLevelsToDrag.Num())
	{
		TSharedPtr<WorldHierarchy::FWorldBrowserDragDropOp> Op = WorldHierarchy::FWorldBrowserDragDropOp::New(StreamingLevelsToDrag);
		return Op;
	}

	return FLevelCollectionModel::CreateDragDropOp();
}

bool FWorldTileCollectionModel::PassesAllFilters(const FLevelModel& Item) const
{
	const FWorldTileModel& Tile = static_cast<const FWorldTileModel&>(Item);
	if (!Tile.IsInLayersList(SelectedLayers))
	{
		return false;
	}
	
	return FLevelCollectionModel::PassesAllFilters(Item);
}

void FWorldTileCollectionModel::BuildWorldCompositionMenu(FMenuBuilder& InMenuBuilder) const
{
	const FLevelCollectionCommands& Commands = FLevelCollectionCommands::Get();
	
	if (!AreAnyLevelsSelected()) // No selection 
	{
		// option to reset world origin
		if (IsOriginRebasingEnabled())
		{
			InMenuBuilder.AddMenuEntry(Commands.ResetWorldOrigin);
		}
	
	}
	else
	{
		// General Levels commands
		InMenuBuilder.BeginSection("Levels", LOCTEXT("LevelsHeader", "Levels") );
		{
			// Make level current
			if (IsOneLevelSelected())
			{
				InMenuBuilder.AddMenuEntry( Commands.World_MakeLevelCurrent );
			}

			// Load/Unload/Save
			InMenuBuilder.AddMenuEntry(Commands.World_LoadLevel);
			InMenuBuilder.AddMenuEntry(Commands.World_UnloadLevel);
			InMenuBuilder.AddMenuEntry(Commands.World_SaveSelectedLevels);
		
			// Editor Visibility commands
			InMenuBuilder.AddSubMenu( 
				LOCTEXT("EditorVisibilityHeader", "Visibility in Editor"),
				LOCTEXT("EditorVisibilitySubMenu_ToolTip", "Selected Level(s) visibility commands for editor worlds"),
				FNewMenuDelegate::CreateSP(const_cast<FWorldTileCollectionModel*>(this), &FWorldTileCollectionModel::FillEditorVisibilitySubMenu ) );

			// Game Visibility commands
			InMenuBuilder.AddSubMenu( 
				LOCTEXT("GameVisibilityHeader", "Visibility in Game"),
				LOCTEXT("GameVisibilitySubMenu_ToolTip", "Selected Level(s) visibility commands for game worlds"),
				FNewMenuDelegate::CreateSP(const_cast<FWorldTileCollectionModel*>(this), &FWorldTileCollectionModel::FillGameVisibilitySubMenu ) );

			// Lock commands
			InMenuBuilder.AddSubMenu( 
				LOCTEXT("LockHeader", "Lock"),
				LOCTEXT("LockSubMenu_ToolTip", "Selected Level(s) lock commands"),
				FNewMenuDelegate::CreateSP(const_cast<FWorldTileCollectionModel*>(this), &FWorldTileCollectionModel::FillLockSubMenu ) );

			InMenuBuilder.AddMenuEntry(Commands.World_FindInContentBrowser);
		}
		InMenuBuilder.EndSection();

		// Assign to layer
		if (AreAnySelectedLevelsEditable())
		{
			InMenuBuilder.BeginSection("Menu_LayersSection");
			{
				InMenuBuilder.AddSubMenu( 
					LOCTEXT("Layer_Assign", "Assign to Layer"),
					FText::GetEmpty(),
					FNewMenuDelegate::CreateSP(const_cast<FWorldTileCollectionModel*>(this), &FWorldTileCollectionModel::FillLayersSubMenu));
			}
			InMenuBuilder.EndSection();
		}

		// Origin
		InMenuBuilder.BeginSection("Menu_LevelOriginSection");
		{
			// Reset level position
			InMenuBuilder.AddMenuEntry(Commands.ResetLevelOrigin);
		
			// Move world orign to level position
			if (IsOneLevelSelected() && IsOriginRebasingEnabled())
			{
				InMenuBuilder.AddMenuEntry(Commands.MoveWorldOrigin);
			}
		}
		InMenuBuilder.EndSection();

		// Level actors selection commands
		InMenuBuilder.BeginSection("Actors", LOCTEXT("ActorsHeader", "Actors") );
		{
			InMenuBuilder.AddMenuEntry( Commands.AddsActors );
			InMenuBuilder.AddMenuEntry( Commands.RemovesActors );

			if (IsOneLevelSelected())
			{
				InMenuBuilder.AddMenuEntry(Commands.MoveActorsToSelected);
				InMenuBuilder.AddMenuEntry(Commands.MoveFoliageToSelected);
			}
		}
		InMenuBuilder.EndSection();

		// Landscape specific stuff
		const bool bCanReimportTiledLandscape = CanReimportTiledlandscape();
		const bool bCanAddAdjacentLandscape = CanAddLandscapeProxy(FWorldTileModel::EWorldDirections::Any);
		if (bCanReimportTiledLandscape || bCanAddAdjacentLandscape)
		{
			InMenuBuilder.BeginSection("Menu_LandscapeSection", LOCTEXT("Menu_LandscapeSectionTitle", "Landscape"));
		
			// Adjacent landscape
			if (bCanAddAdjacentLandscape)
			{
				InMenuBuilder.AddSubMenu( 
					LOCTEXT("AddLandscapeLevel", "Add Adjacent Landscape Level"),
					FText::GetEmpty(),
					FNewMenuDelegate::CreateSP(const_cast<FWorldTileCollectionModel*>(this), &FWorldTileCollectionModel::FillAdjacentLandscapeSubMenu));
			}

			// Tiled landscape
			if (bCanReimportTiledLandscape)
			{
				InMenuBuilder.AddSubMenu( 
					LOCTEXT("ReimportTiledLandscape", "Reimport Tiled Landscape"),
					FText::GetEmpty(),
					FNewMenuDelegate::CreateSP(const_cast<FWorldTileCollectionModel*>(this), &FWorldTileCollectionModel::FillReimportTiledLandscapeSubMenu));
			}
		
			InMenuBuilder.EndSection();
		}
	}

	// Composition section
	InMenuBuilder.BeginSection("Menu_CompositionSection", LOCTEXT("Menu_CompositionSectionTitle", "Composition"));
		InMenuBuilder.AddMenuEntry(Commands.LockTilesLocation); // Lock location
	InMenuBuilder.EndSection();
}

void FWorldTileCollectionModel::BuildHierarchyMenu(FMenuBuilder& InMenuBuilder, EBuildHierarchyMenuFlags Flags) const
{
	const FLevelCollectionCommands& Commands = FLevelCollectionCommands::Get();
		
	// Add common commands
	InMenuBuilder.BeginSection("Levels", LOCTEXT("LevelsHeader", "Levels") );
	{
		// Make level current
		if (IsOneLevelSelected())
		{
			InMenuBuilder.AddMenuEntry( Commands.World_MakeLevelCurrent );
		}

		// Load/Unload/Save
		InMenuBuilder.AddMenuEntry(Commands.World_LoadLevel);
		InMenuBuilder.AddMenuEntry(Commands.World_UnloadLevel);
		InMenuBuilder.AddMenuEntry(Commands.World_SaveSelectedLevels);
				
		// Visibility commands
		InMenuBuilder.AddSubMenu( 
			LOCTEXT("EditorVisibilityHeader", "Visibility in Editor"),
			LOCTEXT("EditorVisibilitySubMenu_ToolTip", "Selected Level(s) visibility commands for editor worlds"),
			FNewMenuDelegate::CreateSP(const_cast<FWorldTileCollectionModel*>(this), &FWorldTileCollectionModel::FillEditorVisibilitySubMenu ) );

		// Game Visibility commands
		if (EnumHasAnyFlags(Flags, EBuildHierarchyMenuFlags::ShowGameVisibility))
		{
			InMenuBuilder.AddSubMenu( 
	            LOCTEXT("GameVisibilityHeader", "Visibility in Game"),
	            LOCTEXT("GameVisibilitySubMenu_ToolTip", "Selected Level(s) visibility commands for game worlds"),
	            FNewMenuDelegate::CreateSP(const_cast<FWorldTileCollectionModel*>(this), &FWorldTileCollectionModel::FillGameVisibilitySubMenu ) );
		}
		
		// Lock commands
		InMenuBuilder.AddSubMenu( 
			LOCTEXT("LockHeader", "Lock"),
			LOCTEXT("LockSubMenu_ToolTip", "Selected Level(s) lock commands"),
			FNewMenuDelegate::CreateSP(const_cast<FWorldTileCollectionModel*>(this), &FWorldTileCollectionModel::FillLockSubMenu ) );
	
		InMenuBuilder.AddMenuEntry(Commands.World_FindInContentBrowser);
	}
	InMenuBuilder.EndSection();

	// Assign to layer
	if (AreAnySelectedLevelsEditable())
	{
		InMenuBuilder.BeginSection("Menu_LayersSection");
		{
			InMenuBuilder.AddSubMenu( 
				LOCTEXT("WorldLayers", "Assign to Layer"),
				FText::GetEmpty(),
				FNewMenuDelegate::CreateSP(const_cast<FWorldTileCollectionModel*>(this), &FWorldTileCollectionModel::FillLayersSubMenu)
			);
		}
		InMenuBuilder.EndSection();
	}

	// Hierarchy management
	if (AreAnyLevelsSelected())
	{
		InMenuBuilder.BeginSection("Menu_HierarchySection");
		{
			InMenuBuilder.AddMenuEntry(Commands.ExpandSelectedItems);
			InMenuBuilder.AddMenuEntry(Commands.ClearParentLink);
		}
		InMenuBuilder.EndSection();
	}
		
	// Level selection commands
	InMenuBuilder.BeginSection("LevelsSelection", LOCTEXT("SelectionHeader", "Selection") );
	{
		InMenuBuilder.AddMenuEntry( Commands.SelectAllLevels );
		InMenuBuilder.AddMenuEntry( Commands.DeselectAllLevels );
		InMenuBuilder.AddMenuEntry( Commands.InvertLevelSelection );
	}
	InMenuBuilder.EndSection();

	// Level actors selection commands
	InMenuBuilder.BeginSection("Actors", LOCTEXT("ActorsHeader", "Actors") );
	{
		InMenuBuilder.AddMenuEntry( Commands.AddsActors );
		InMenuBuilder.AddMenuEntry( Commands.RemovesActors );

		// Move selected actors to a selected level
		if (IsOneLevelSelected())
		{
			InMenuBuilder.AddMenuEntry( Commands.MoveActorsToSelected );
			InMenuBuilder.AddMenuEntry( Commands.MoveFoliageToSelected );
		}
	}
	InMenuBuilder.EndSection();
}

void FWorldTileCollectionModel::FillLayersSubMenu(FMenuBuilder& InMenuBuilder) const
{
	for (auto It = AllLayers.CreateConstIterator(); It; ++It)
	{
		InMenuBuilder.AddMenuEntry(
			FText::FromString((*It).Name), 
			FText(), FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(
				const_cast<FWorldTileCollectionModel*>(this), &FWorldTileCollectionModel::AssignSelectedLevelsToLayer_Executed, (*It)
				)
			)
		);
	}
}

void FWorldTileCollectionModel::FillAdjacentLandscapeSubMenu(FMenuBuilder& InMenuBuilder) const
{
	const FLevelCollectionCommands& Commands = FLevelCollectionCommands::Get();

	InMenuBuilder.AddMenuEntry(Commands.AddLandscapeLevelXNegative, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "WorldBrowser.DirectionXNegative"));
	InMenuBuilder.AddMenuEntry(Commands.AddLandscapeLevelXPositive, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "WorldBrowser.DirectionXPositive"));
	InMenuBuilder.AddMenuEntry(Commands.AddLandscapeLevelYNegative, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "WorldBrowser.DirectionYNegative"));
	InMenuBuilder.AddMenuEntry(Commands.AddLandscapeLevelYPositive, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "WorldBrowser.DirectionYPositive"));
}

void FWorldTileCollectionModel::FillReimportTiledLandscapeSubMenu(FMenuBuilder& InMenuBuilder) const
{
	const FLevelCollectionCommands& Commands = FLevelCollectionCommands::Get();

	// Add "Heightmap" menu entry
	InMenuBuilder.AddMenuEntry(
			LOCTEXT("Menu_HeightmapTitle", "Heightmap"), 
			FText(), FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(
				const_cast<FWorldTileCollectionModel*>(this), &FWorldTileCollectionModel::ReimportTiledLandscape_Executed, HeightmapLayerName
				)
			)
		);

	// Weightmaps	
	InMenuBuilder.AddSubMenu( 
				LOCTEXT("Menu_WeightmapsTitle", "Weightmaps"),
				FText::GetEmpty(),
				FNewMenuDelegate::CreateSP(this, &FWorldTileCollectionModel::FillWeightmapsSubMenu)
			);
}

void FWorldTileCollectionModel::FillWeightmapsSubMenu(FMenuBuilder& InMenuBuilder) const
{
	// Add "All Weightmaps" menu entry
	InMenuBuilder.AddMenuEntry(
			LOCTEXT("Menu_AllWeightmapsTitle", "All Weightmaps"), 
			FText(), FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(
				const_cast<FWorldTileCollectionModel*>(this), &FWorldTileCollectionModel::ReimportTiledLandscape_Executed, FName(NAME_None)
				)
			)
		);
	
	TArray<FName> LayerNames;
	// Get list of the landscape layers
	for (const auto& LevelModel : SelectedLevelsList)
	{
		auto TileModel = StaticCastSharedPtr<FWorldTileModel>(LevelModel);
		if (TileModel->IsLandscapeBased())
		{
			TArray<FName> Layers = UE::Landscape::RetrieveTargetLayerNamesFromMaterial(TileModel->GetLandscape()->LandscapeMaterial);
			for (FName LayerName : Layers)
			{
				LayerNames.AddUnique(LayerName);
			}
		}
	}

	for (FName LayerName : LayerNames)
	{
		InMenuBuilder.AddMenuEntry(
			FText::FromName(LayerName), 
			FText(), FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(
				const_cast<FWorldTileCollectionModel*>(this), &FWorldTileCollectionModel::ReimportTiledLandscape_Executed, LayerName
				)
			)
		);
	}
}

void FWorldTileCollectionModel::CustomizeFileMainMenu(FMenuBuilder& InMenuBuilder) const
{
	FLevelCollectionModel::CustomizeFileMainMenu(InMenuBuilder);

	const FLevelCollectionCommands& Commands = FLevelCollectionCommands::Get();
		
	InMenuBuilder.BeginSection("LevelsAddLevel");
	{
		InMenuBuilder.AddMenuEntry(Commands.World_CreateNewLevel);
		InMenuBuilder.AddMenuEntry(Commands.ImportTiledLandscape);
	}
	InMenuBuilder.EndSection();
}

bool FWorldTileCollectionModel::GetPlayerView(FVector& Location, FRotator& Rotation) const
{
	if (IsSimulating())
	{
		UWorld* SimulationWorld = GetSimulationWorld();
		for (FConstPlayerControllerIterator Iterator = SimulationWorld->GetPlayerControllerIterator(); Iterator; ++Iterator)
		{
			if (APlayerController* PlayerActor = Iterator->Get())
			{
				PlayerActor->GetPlayerViewPoint(Location, Rotation);
				return true;
			}
		}
	}
	
	return false;
}

bool FWorldTileCollectionModel::GetObserverView(FVector& Location, FRotator& Rotation) const
{
	// We are in the SIE
	if (GEditor->bIsSimulatingInEditor && GCurrentLevelEditingViewportClient->IsSimulateInEditorViewport())
	{
		Rotation = GCurrentLevelEditingViewportClient->GetViewRotation();
		Location = GCurrentLevelEditingViewportClient->GetViewLocation();
		return true;
	}

	// We are in the editor world
	if (GEditor->PlayWorld == nullptr)
	{
		for (const FLevelEditorViewportClient* ViewportClient : GEditor->GetLevelViewportClients())
		{
			if (ViewportClient && ViewportClient->IsPerspective())
			{
				Rotation = ViewportClient->GetViewRotation();
				Location = ViewportClient->GetViewLocation();
				return true;
			}
		}
	}

	return false;
}

static double Area(FVector2D InRect)
{
	return (double)InRect.X * (double)InRect.Y;
}

bool FWorldTileCollectionModel::CompareLevelsZOrder(TSharedPtr<FLevelModel> InA, TSharedPtr<FLevelModel> InB) const
{
	TSharedPtr<FWorldTileModel> A = StaticCastSharedPtr<FWorldTileModel>(InA);
	TSharedPtr<FWorldTileModel> B = StaticCastSharedPtr<FWorldTileModel>(InB);

	if (A->TileDetails->ZOrder == B->TileDetails->ZOrder)
	{
		if (A->GetLevelSelectionFlag() == B->GetLevelSelectionFlag())
		{
			return Area(B->GetLevelSize2D()) < Area(A->GetLevelSize2D());
		}
		
		return B->GetLevelSelectionFlag() > A->GetLevelSelectionFlag();
	}

	return B->TileDetails->ZOrder > A->TileDetails->ZOrder;
}

void FWorldTileCollectionModel::RegisterDetailsCustomization(FPropertyEditorModule& InPropertyModule, 
																TSharedPtr<IDetailsView> InDetailsView)
{
	TSharedRef<FWorldTileCollectionModel> WorldModel = StaticCastSharedRef<FWorldTileCollectionModel>(this->AsShared());
	
	// Register our struct customizations
	InPropertyModule.RegisterCustomPropertyTypeLayout("TileStreamingLevelDetails", 
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FStreamingLevelDetailsCustomization::MakeInstance, WorldModel)
		);

	InPropertyModule.RegisterCustomPropertyTypeLayout("TileLODEntryDetails", 
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FTileLODEntryDetailsCustomization::MakeInstance,  WorldModel)
		);

	InDetailsView->RegisterInstancedCustomPropertyLayout(UWorldTileDetails::StaticClass(), 
		FOnGetDetailCustomizationInstance::CreateStatic(&FWorldTileDetailsCustomization::MakeInstance,  WorldModel)
		);
}

void FWorldTileCollectionModel::UnregisterDetailsCustomization(FPropertyEditorModule& InPropertyModule,
															   TSharedPtr<IDetailsView> InDetailsView)
{
	InPropertyModule.UnregisterCustomPropertyTypeLayout("TileStreamingLevelDetails");
	InPropertyModule.UnregisterCustomPropertyTypeLayout("TileLODEntryDetails");
	InDetailsView->UnregisterInstancedCustomPropertyLayout(UWorldTileDetails::StaticClass());
}

TSharedPtr<FWorldTileModel> FWorldTileCollectionModel::GetWorldRootModel()
{
	return StaticCastSharedPtr<FWorldTileModel>(RootLevelsList[0]);
}

static void InvalidateLightingCache(const FLevelModelList& ModelList)
{
	for (auto It = ModelList.CreateConstIterator(); It; ++It)
	{
		ULevel* Level = (*It)->GetLevelObject();
		if (Level && Level->bIsVisible)
		{
			for (auto ActorIt = Level->Actors.CreateIterator(); ActorIt; ++ActorIt)
			{
				ALight* Light = Cast<ALight>(*ActorIt);
				if (Light)
				{
					Light->InvalidateLightingCache();
				}
			}
		}
	}
}

bool FWorldTileCollectionModel::HasWorldRoot() const
{
	return CurrentWorld->WorldComposition != NULL;
}

void FWorldTileCollectionModel::BindCommands()
{
	FLevelCollectionModel::BindCommands();

	const FLevelCollectionCommands& Commands = FLevelCollectionCommands::Get();
	FUICommandList& ActionList = *CommandList;

	//
	ActionList.MapAction(Commands.World_CreateNewLevel,
		FExecuteAction::CreateSP(this, &FWorldTileCollectionModel::CreateNewLevel_Executed));

	ActionList.MapAction(Commands.ClearParentLink,
		FExecuteAction::CreateSP(this, &FWorldTileCollectionModel::ClearParentLink_Executed),
		FCanExecuteAction::CreateSP(this, &FLevelCollectionModel::AreAnySelectedLevelsEditable));
	
	//
	ActionList.MapAction(Commands.MoveWorldOrigin,
		FExecuteAction::CreateSP(this, &FWorldTileCollectionModel::MoveWorldOrigin_Executed),
		FCanExecuteAction::CreateSP(this, &FWorldTileCollectionModel::IsOneLevelSelected));

	ActionList.MapAction(Commands.ResetWorldOrigin,
		FExecuteAction::CreateSP(this, &FWorldTileCollectionModel::ResetWorldOrigin_Executed));

	ActionList.MapAction(Commands.ResetLevelOrigin,
		FExecuteAction::CreateSP(this, &FWorldTileCollectionModel::ResetLevelOrigin_Executed),
		FCanExecuteAction::CreateSP(this, &FWorldTileCollectionModel::AreAnySelectedLevelsEditable));


	// Landscape operations
	ActionList.MapAction(Commands.ImportTiledLandscape,
		FExecuteAction::CreateSP(this, &FWorldTileCollectionModel::ImportTiledLandscape_Executed));
		
	ActionList.MapAction(Commands.AddLandscapeLevelXNegative,
		FExecuteAction::CreateSP(this, &FWorldTileCollectionModel::AddLandscapeProxy_Executed, FWorldTileModel::XNegative),
		FCanExecuteAction::CreateSP(this, &FWorldTileCollectionModel::CanAddLandscapeProxy, FWorldTileModel::XNegative));
	
	ActionList.MapAction(Commands.AddLandscapeLevelXPositive,
		FExecuteAction::CreateSP(this, &FWorldTileCollectionModel::AddLandscapeProxy_Executed, FWorldTileModel::XPositive),
		FCanExecuteAction::CreateSP(this, &FWorldTileCollectionModel::CanAddLandscapeProxy, FWorldTileModel::XPositive));
	
	ActionList.MapAction(Commands.AddLandscapeLevelYNegative,
		FExecuteAction::CreateSP(this, &FWorldTileCollectionModel::AddLandscapeProxy_Executed, FWorldTileModel::YNegative),
		FCanExecuteAction::CreateSP(this, &FWorldTileCollectionModel::CanAddLandscapeProxy, FWorldTileModel::YNegative));
	
	ActionList.MapAction(Commands.AddLandscapeLevelYPositive,
		FExecuteAction::CreateSP(this, &FWorldTileCollectionModel::AddLandscapeProxy_Executed, FWorldTileModel::YPositive),
		FCanExecuteAction::CreateSP(this, &FWorldTileCollectionModel::CanAddLandscapeProxy, FWorldTileModel::YPositive));

	ActionList.MapAction(
		Commands.LockTilesLocation,
		FExecuteAction::CreateSP(this, &FWorldTileCollectionModel::OnToggleLockTilesLocation),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &FWorldTileCollectionModel::IsLockTilesLocationEnabled));
}

void FWorldTileCollectionModel::OnLevelsCollectionChanged()
{
	// populate tree structure of the root folder
	StaticTileList.Empty();
	UWorld* ThisWorld = CurrentWorld.Get();
	UWorldComposition* WorldComposition = ThisWorld ? ThisWorld->WorldComposition.Get() : nullptr;
	if (WorldComposition)
	{
		// Force rescanning world composition tiles
		WorldComposition->Rescan(true);
		
		// Initialize root level
		TSharedPtr<FWorldTileModel> RootLevelModel = MakeShareable(new FWorldTileModel(*this, INDEX_NONE));
		RootLevelModel->SetLevelExpansionFlag(true);

		AllLevelsList.Add(RootLevelModel);
		AllLevelsMap.Add(RootLevelModel->TileDetails->PackageName, RootLevelModel);
		RootLevelsList.Add(RootLevelModel);
				
		// Initialize child tiles
		auto TileList = WorldComposition->GetTilesList();
		for (int32 TileIdx = 0; TileIdx < TileList.Num(); ++TileIdx)
		{
			TSharedPtr<FWorldTileModel> TileLevelModel = MakeShareable(new FWorldTileModel(*this, TileIdx));
			
			// Make sure all sub-levels belong to our world
			ULevel* TileLevelObject = TileLevelModel->GetLevelObject();
			if (TileLevelObject && TileLevelObject->OwningWorld && 
				TileLevelObject->OwningWorld != ThisWorld)
			{
				ThisWorld->RemoveFromWorld(TileLevelObject);
				TileLevelObject->OwningWorld = ThisWorld;
			}
			
			AllLevelsList.Add(TileLevelModel);
			AllLevelsMap.Add(TileLevelModel->TileDetails->PackageName, TileLevelModel);
		}
		
		SetupParentChildLinks();
		GetWorldRootModel()->SortRecursive();
		UpdateAllLevels();

		PopulateLayersList();
	}

	FLevelCollectionModel::OnLevelsCollectionChanged();

	// Sync levels selection to world
	SetSelectedLevelsFromWorld();
}

void FWorldTileCollectionModel::PopulateLayersList()
{
	AllLayers.Empty();
	SelectedLayers.Empty();

	if (HasWorldRoot())
	{
		AllLayers.Append(ManagedLayers);
		
		for (auto It = AllLevelsList.CreateIterator(); It; ++It)
		{
			TSharedPtr<FWorldTileModel> TileModel = StaticCastSharedPtr<FWorldTileModel>(*It);
			AllLayers.AddUnique(TileModel->TileDetails->Layer);
		}
	}
}

void FWorldTileCollectionModel::SetupParentChildLinks()
{
	// purge current hierarchy
	for (auto It = AllLevelsList.CreateIterator(); It; ++It)
	{
		(*It)->SetParent(NULL);
		(*It)->RemoveAllChildren();
	}
	
	// Setup->parent child links
	for (auto It = AllLevelsList.CreateIterator(); It; ++It)
	{
		TSharedPtr<FWorldTileModel> TileModel = StaticCastSharedPtr<FWorldTileModel>(*It);
		if (!TileModel->IsRootTile())
		{
			TSharedPtr<FLevelModel> ParentModel = FindLevelModel(TileModel->TileDetails->ParentPackageName);

			if (!ParentModel.IsValid())
			{
				// All parentless tiles will be attached to a root tile
				ParentModel = GetWorldRootModel();
			}

			ParentModel->AddChild(TileModel);
			TileModel->SetParent(ParentModel);
		}
	}
}

bool FWorldTileCollectionModel::AreAnyLayersSelected() const
{
	return SelectedLayers.Num() > 0;
}

void FWorldTileCollectionModel::OnLevelsSelectionChanged()
{
	// Update list of levels which are not affected by current selection (not in selection list and not among children of selected levels)
	TSet<TSharedPtr<FLevelModel>> A; A.Append(GetLevelsHierarchy(GetSelectedLevels()));
	TSet<TSharedPtr<FLevelModel>> B; B.Append(AllLevelsList);
	StaticTileList = B.Difference(A).Array();
		
	FLevelCollectionModel::OnLevelsSelectionChanged();
}

void FWorldTileCollectionModel::OnLevelsHierarchyChanged()
{
	GetWorldRootModel()->SortRecursive();
	FLevelCollectionModel::OnLevelsHierarchyChanged();
}

void FWorldTileCollectionModel::OnPreLoadLevels(const FLevelModelList& InList)
{
	// Compute focus area for loading levels
	FBox FocusArea(ForceInit);
	for (auto It = InList.CreateConstIterator(); It; ++It)
	{
		TSharedPtr<FWorldTileModel> TileModel = StaticCastSharedPtr<FWorldTileModel>(*It);
		
		FBox ResultBox = FocusArea + TileModel->GetLevelBounds();
		if (!ResultBox.IsValid ||
			(ResultBox.GetExtent().X < EditableAxisLength() && ResultBox.GetExtent().Y < EditableAxisLength()))
		{
			FocusArea = ResultBox;
		}
	}

	Focus(FocusArea, OriginAtCenter);
}

void FWorldTileCollectionModel::OnPreShowLevels(const FLevelModelList& InList)
{
	// Make sure requested levels will fit to the world 
	Focus(GetLevelsBoundingBox(InList, false), EnsureEditableCentered);
}

void FWorldTileCollectionModel::DeselectLevels(const FWorldTileLayer& InLayer)
{
	for (int32 LevelIdx = SelectedLevelsList.Num() - 1; LevelIdx >= 0; --LevelIdx)
	{
		TSharedPtr<FWorldTileModel> TileModel = StaticCastSharedPtr<FWorldTileModel>(SelectedLevelsList[LevelIdx]);
		
		if (TileModel->TileDetails->Layer == InLayer)
		{
			SelectedLevelsList.RemoveAt(LevelIdx);
		}
	}
}

const TMap<FName, int32>& FWorldTileCollectionModel::GetPreviewStreamingLevels() const
{
	return PreviewVisibleTiles;
}

void FWorldTileCollectionModel::UpdateStreamingPreview(FVector2D InLocation, bool bEnabled)
{
	if (bEnabled)
	{
		FVector NewPreviewLocation = FVector(InLocation, 0);
		
		if ((PreviewLocation-NewPreviewLocation).SizeSquared() > FMath::Square(KINDA_SMALL_NUMBER))
		{
			PreviewLocation = NewPreviewLocation;
			PreviewVisibleTiles.Empty();
			
			// Add levels which is visible due to distance based streaming
			TArray<FDistanceVisibleLevel> DistanceVisibleLevels;
			TArray<FDistanceVisibleLevel> DistanceHiddenLevels;
			GetWorld()->WorldComposition->GetDistanceVisibleLevels(PreviewLocation, DistanceVisibleLevels, DistanceHiddenLevels);

			for (const auto& VisibleLevel : DistanceVisibleLevels)
			{
				PreviewVisibleTiles.Add(VisibleLevel.StreamingLevel->GetWorldAssetPackageFName(), VisibleLevel.LODIndex);
			}
		}
	}
	else
	{
		PreviewVisibleTiles.Empty();
	}
}

bool FWorldTileCollectionModel::AddLevelToTheWorld(const TSharedPtr<FWorldTileModel>& InLevel)
{
	if (InLevel.IsValid() && InLevel->GetLevelObject())
	{
		// Make level visible only if it is inside editable world area
		if (InLevel->ShouldBeVisible(EditableWorldArea()))
		{
			// do not add already visible levels
			if (InLevel->GetLevelObject()->bIsVisible == false)
			{
				GetWorld()->AddToWorld(InLevel->GetLevelObject());
			}
		}
		else
		{
			// Make sure level is in Persistent world levels list
			GetWorld()->AddLevel(InLevel->GetLevelObject());
			InLevel->Shelve();
		}

		return true;
	}	

	return false;
}

void FWorldTileCollectionModel::ShelveLevels(const FWorldTileModelList& InLevels)
{
	for (auto It = InLevels.CreateConstIterator(); It; ++It)
	{
		(*It)->Shelve();
	}
}

void FWorldTileCollectionModel::UnshelveLevels(const FWorldTileModelList& InLevels)
{
	for (auto It = InLevels.CreateConstIterator(); It; ++It)
	{
		(*It)->Unshelve();
	}
}

TSharedPtr<FLevelModel> FWorldTileCollectionModel::CreateNewEmptyLevel()
{
	TSharedPtr<FLevelModel> NewLevelModel;

	if (IsReadOnly())
	{
		return NewLevelModel;
	}
	
	GLevelEditorModeTools().ActivateDefaultMode();

	// Save new level to the same directory where selected level/folder is
	FString Directory = FPaths::GetPath(GetWorldRootModel()->GetPackageFileName());
	if (SelectedLevelsList.Num() > 0)
	{
		Directory = FPaths::GetPath(SelectedLevelsList[0]->GetPackageFileName());
	}
			
	// Create a new world - so we can 'borrow' its level
	UWorld* NewGWorld = UWorld::CreateWorld(EWorldType::None, false );
	check(NewGWorld);

	// Save the last directory
	FString OldLastDirectory = FEditorDirectories::Get().GetLastDirectory(ELastDirectory::UNR);
	// Temporally change last directory to our path	
	FEditorDirectories::Get().SetLastDirectory(ELastDirectory::UNR, Directory);
	// Save new empty level
	bool bSaved = FEditorFileUtils::SaveLevel(NewGWorld->PersistentLevel);
	// Restore last directory	
	FEditorDirectories::Get().SetLastDirectory(ELastDirectory::UNR, OldLastDirectory);

	// Update levels list
	if (bSaved)
	{
		PopulateLevelsList();
		NewLevelModel = FindLevelModel(NewGWorld->GetOutermost()->GetFName());
	}
	
	// Destroy the new world we created and collect the garbage
	NewGWorld->DestroyWorld( false );
	CollectGarbage( GARBAGE_COLLECTION_KEEPFLAGS );
		
	return NewLevelModel;
}

void FWorldTileCollectionModel::Focus(FBox InArea, FocusStrategy InStrategy)
{
	if (IsReadOnly() || !InArea.IsValid || !IsOriginRebasingEnabled())
	{
		return;
	}
	
	const bool bIsEditable = EditableWorldArea().IsInsideXY(InArea);
		
	switch (InStrategy)
	{
	case OriginAtCenter:
		{
			FIntVector OriginOffset = FIntVector(InArea.GetCenter().X, InArea.GetCenter().Y, 0);
			GetWorld()->SetNewWorldOrigin(GetWorld()->OriginLocation + OriginOffset);
		}
		break;
	
	case EnsureEditableCentered:
		if (!bIsEditable)
		{
			FIntVector OriginOffset = FIntVector(InArea.GetCenter().X, InArea.GetCenter().Y, 0);
			GetWorld()->SetNewWorldOrigin(GetWorld()->OriginLocation + OriginOffset);
		}
		break;

	case EnsureEditable:
		if (!bIsEditable)
		{
			InArea = InArea.ExpandBy(InArea.GetExtent().Size2D()*0.1f);
			FBox NewWorldBounds = EditableWorldArea();
			
			if (InArea.Min.X < NewWorldBounds.Min.X)
			{
				NewWorldBounds.Min.X = InArea.Min.X;
				NewWorldBounds.Max.X = InArea.Min.X + EditableAxisLength();
			}

			if (InArea.Min.Y < NewWorldBounds.Min.Y)
			{
				NewWorldBounds.Min.Y = InArea.Min.Y;
				NewWorldBounds.Max.Y = InArea.Min.Y + EditableAxisLength();
			}

			if (InArea.Max.X > NewWorldBounds.Max.X)
			{
				NewWorldBounds.Max.X = InArea.Max.X;
				NewWorldBounds.Min.X = InArea.Max.X - EditableAxisLength();
			}

			if (InArea.Max.Y > NewWorldBounds.Max.Y)
			{
				NewWorldBounds.Max.Y = InArea.Max.Y;
				NewWorldBounds.Min.Y = InArea.Max.Y - EditableAxisLength();
			}

			FIntVector OriginOffset = FIntVector(NewWorldBounds.GetCenter().X, NewWorldBounds.GetCenter().Y, 0);
			GetWorld()->SetNewWorldOrigin(GetWorld()->OriginLocation + OriginOffset);
		}
		break;
	}
}

void FWorldTileCollectionModel::PreWorldOriginOffset(UWorld* InWorld, FIntVector InSrcOrigin, FIntVector InDstOrigin)
{
	// Make sure we handle our world notifications
	if (GetWorld() != InWorld)
	{
		return;
	}
	
	FBox NewWorldBounds = EditableWorldArea().ShiftBy(FVector(InDstOrigin - InSrcOrigin));

	// Shelve levels which do not fit to a new world bounds
	for (auto It = AllLevelsList.CreateIterator(); It; ++It)
	{
		TSharedPtr<FWorldTileModel> TileModel = StaticCastSharedPtr<FWorldTileModel>(*It);
		if (TileModel->ShouldBeVisible(NewWorldBounds) == false)
		{
			TileModel->Shelve();
		}
	}
}

void FWorldTileCollectionModel::PostWorldOriginOffset(UWorld* InWorld, FIntVector InSrcOrigin, FIntVector InDstOrigin)
{
	// Make sure we handle our world notifications
	if (GetWorld() != InWorld)
	{
		return;
	}

	FBox CurrentWorldBounds = EditableWorldArea();

	// Unshelve levels which do fit to current world bounds
	for (auto It = AllLevelsList.CreateIterator(); It; ++It)
	{
		TSharedPtr<FWorldTileModel> TileModel = StaticCastSharedPtr<FWorldTileModel>(*It);
		if (TileModel->ShouldBeVisible(CurrentWorldBounds))
		{
			TileModel->Unshelve();
		}
	}
}

bool FWorldTileCollectionModel::AreAnySelectedLevelsHaveLandscape() const
{
	for (auto LevelModel : SelectedLevelsList)
	{
		if (LevelModel->IsLoaded() && StaticCastSharedPtr<FWorldTileModel>(LevelModel)->IsLandscapeBased())
		{
			return true;
		}
	}

	return false;
}

FVector2D FWorldTileCollectionModel::SnapTranslationDelta(const FLevelModelList& InLevels, FVector2D InTranslationDelta, bool bBoundsSnapping, FVector2D::FReal InSnappingValue)
{
	for (auto It = InLevels.CreateConstIterator(); It; ++It)
	{
		TSharedPtr<FWorldTileModel> TileModel = StaticCastSharedPtr<FWorldTileModel>(*It);
		if (TileModel->IsLandscapeBased())
		{
			return SnapTranslationDeltaLandscape(TileModel, InTranslationDelta, InSnappingValue);
		}
	}

	// grid snapping
	if (!bBoundsSnapping)
	{
		InSnappingValue = FMath::Max(0.f, InSnappingValue);
		InTranslationDelta.X = FMath::GridSnap(InTranslationDelta.X, InSnappingValue);
		InTranslationDelta.Y = FMath::GridSnap(InTranslationDelta.Y, InSnappingValue);
		return InTranslationDelta;
	}

	//
	// Bounds snapping
	//
				
	// Compute moving levels total bounding box
	FBox MovingLevelsBBoxStart = GetLevelsBoundingBox(InLevels, true);
	FBox MovingLevelsBBoxExpected = MovingLevelsBBoxStart.ShiftBy(FVector(InTranslationDelta, 0.f));
			
	// Expand moving box by maximum snapping distance, so we can find all static levels we touching
	FBox TestLevelsBBox = MovingLevelsBBoxExpected.ExpandBy(InSnappingValue);
	
	FVector2D ClosestValue(FLT_MAX, FLT_MAX);
	FVector2D MinDistance(FLT_MAX, FLT_MAX);
	// Stores which box side is going to be snapped 
	FVector2D BoxSide(	MovingLevelsBBoxExpected.Min.X, 
						MovingLevelsBBoxExpected.Min.Y);
	
	// Test axis values
	float TestPointsX1[4] = {	(float)MovingLevelsBBoxExpected.Min.X, 
								(float)MovingLevelsBBoxExpected.Min.X, 
								(float)MovingLevelsBBoxExpected.Max.X, 
								(float)MovingLevelsBBoxExpected.Max.X 
	};

	float TestPointsY1[4] = {	(float)MovingLevelsBBoxExpected.Min.Y, 
								(float)MovingLevelsBBoxExpected.Min.Y, 
								(float)MovingLevelsBBoxExpected.Max.Y, 
								(float)MovingLevelsBBoxExpected.Max.Y 
	};
	
	for (auto It = StaticTileList.CreateConstIterator(); It; ++It)
	{
		TSharedPtr<FWorldTileModel> StaticTileModel = StaticCastSharedPtr<FWorldTileModel>(*It);
		FBox StaticLevelBBox = StaticTileModel->GetLevelBounds();
		
		if (StaticLevelBBox.IntersectXY(TestLevelsBBox) || 
			StaticLevelBBox.IsInsideXY(TestLevelsBBox) ||
			TestLevelsBBox.IsInsideXY(StaticLevelBBox))
		{

			// Find closest X value
			FVector::FReal TestPointsX2[4] = {	(float)StaticLevelBBox.Min.X, 
										(float)StaticLevelBBox.Max.X, 
										(float)StaticLevelBBox.Min.X, 
										(float)StaticLevelBBox.Max.X 
			};

			for (int32 i = 0; i < 4; i++)
			{
				float Distance = FMath::Abs(TestPointsX2[i] - TestPointsX1[i]);
				if (Distance < MinDistance.X)
				{
					MinDistance.X	= Distance;
					ClosestValue.X	= TestPointsX2[i];
					BoxSide.X		= TestPointsX1[i];
				}
			}
			
			// Find closest Y value
			float TestPointsY2[4] = {	(float)StaticLevelBBox.Min.Y, 
										(float)StaticLevelBBox.Max.Y, 
										(float)StaticLevelBBox.Min.Y, 
										(float)StaticLevelBBox.Max.Y 
			};

			for (int32 i = 0; i < 4; i++)
			{
				float Distance = FMath::Abs(TestPointsY2[i] - TestPointsY1[i]);
				if (Distance < MinDistance.Y)
				{
					MinDistance.Y	= Distance;
					ClosestValue.Y	= TestPointsY2[i];
					BoxSide.Y		= TestPointsY1[i];
				}
			}
		}
	}

	// Snap by X value
	if (MinDistance.X < InSnappingValue)
	{
		float Difference = ClosestValue.X - BoxSide.X;
		MovingLevelsBBoxExpected.Min.X+= Difference;
		MovingLevelsBBoxExpected.Max.X+= Difference;
	}
	
	// Snap by Y value
	if (MinDistance.Y < InSnappingValue)
	{
		float Difference = ClosestValue.Y - BoxSide.Y;
		MovingLevelsBBoxExpected.Min.Y+= Difference;
		MovingLevelsBBoxExpected.Max.Y+= Difference;
	}
	
	// Calculate final snapped delta
	FVector Delta = MovingLevelsBBoxExpected.GetCenter() - MovingLevelsBBoxStart.GetCenter();
	return FIntPoint(Delta.X, Delta.Y);
}

FVector2D FWorldTileCollectionModel::SnapTranslationDeltaLandscape(const TSharedPtr<FWorldTileModel>& LandscapeTile, 
																	FVector2D InAbsoluteDelta, 
																	float SnappingDistance)
{
	ALandscapeProxy* Landscape = LandscapeTile->GetLandscape();
	FVector ComponentScale = Landscape->GetRootComponent()->GetRelativeScale3D()*Landscape->ComponentSizeQuads;
	
	return FVector2D(	FMath::GridSnap(InAbsoluteDelta.X, ComponentScale.X),
						FMath::GridSnap(InAbsoluteDelta.Y, ComponentScale.Y));
}

TArray<FWorldTileLayer>& FWorldTileCollectionModel::GetLayers()
{
	return AllLayers;
}

void FWorldTileCollectionModel::AddLayer(const FWorldTileLayer& InLayer)
{
	if (IsReadOnly())
	{
		return;
	}
	
	AllLayers.AddUnique(InLayer);
}

void FWorldTileCollectionModel::AddManagedLayer(const FWorldTileLayer& InLayer)
{
	if (IsReadOnly())
	{
		return;
	}
	
	ManagedLayers.AddUnique(InLayer);
	AllLayers.AddUnique(InLayer);
}

void FWorldTileCollectionModel::SetSelectedLayer(const FWorldTileLayer& InLayer)
{
	SelectedLayers.Empty();
	SelectedLayers.Add(InLayer);
	OnFilterChanged();

	// reset levels selection
	SetSelectedLevels(FLevelModelList());
}

void FWorldTileCollectionModel::SetSelectedLayers(const TArray<FWorldTileLayer>& InLayers)
{
	SelectedLayers.Empty();
	for (int32 LayerIndex = 0; LayerIndex < InLayers.Num(); ++LayerIndex)
	{
		SelectedLayers.AddUnique(InLayers[LayerIndex]);
	}

	OnFilterChanged();

	// reset levels selection
	SetSelectedLevels(FLevelModelList());
}

void FWorldTileCollectionModel::ToggleLayerSelection(const FWorldTileLayer& InLayer)
{
	if (IsLayerSelected(InLayer))
	{
		SelectedLayers.Remove(InLayer);
		OnFilterChanged();
		// deselect Levels which belongs to this layer
		DeselectLevels(InLayer);
	}
	else
	{
		SelectedLayers.Add(InLayer);
		OnFilterChanged();
	}
}

bool FWorldTileCollectionModel::IsLayerSelected(const FWorldTileLayer& InLayer)
{
	return SelectedLayers.Contains(InLayer);
}

void FWorldTileCollectionModel::CreateNewLevel_Executed()
{
	CreateNewEmptyLevel();
}

void FWorldTileCollectionModel::AssignSelectedLevelsToLayer_Executed(FWorldTileLayer InLayer)
{
	if (IsReadOnly())
	{
		return;
	}
	
	for(auto It = SelectedLevelsList.CreateConstIterator(); It; ++It)
	{
		StaticCastSharedPtr<FWorldTileModel>(*It)->AssignToLayer(InLayer);
	}

	PopulateFilteredLevelsList();
}

void FWorldTileCollectionModel::ClearParentLink_Executed()
{	
	for(auto It = SelectedLevelsList.CreateConstIterator(); It; ++It)
	{
		(*It)->AttachTo(GetWorldRootModel());
	}
	BroadcastHierarchyChanged();
}

bool FWorldTileCollectionModel::CanAddLandscapeProxy(FWorldTileModel::EWorldDirections InWhere) const
{
	if (SelectedLevelsList.Num() == 1 &&
		SelectedLevelsList[0]->IsVisibleInEditor() &&
		StaticCastSharedPtr<FWorldTileModel>(SelectedLevelsList[0])->IsLandscapeBased())
	{
		return true;
	}
	
	return false;
}

bool FWorldTileCollectionModel::CanReimportTiledlandscape() const
{
	for (const auto& LevelModel : SelectedLevelsList)
	{
		if (LevelModel->IsEditable() && StaticCastSharedPtr<FWorldTileModel>(LevelModel)->CanReimportHeightmap())
		{
			return true;
		}
	}

	return false;
}

void FWorldTileCollectionModel::AddLandscapeProxy_Executed(FWorldTileModel::EWorldDirections InWhere)
{
	if (IsReadOnly() || !IsOneLevelSelected())
	{
		return;
	}
	
	// We expect there is a landscape based level selected, sp we can create new landscape level based on this
	TSharedPtr<FWorldTileModel> LandscapeTileModel = StaticCastSharedPtr<FWorldTileModel>(SelectedLevelsList[0]);
	if (!LandscapeTileModel->IsLandscapeBased())
	{
		return;
	}
	
	// Create new empty level for landscape proxy
	TSharedPtr<FWorldTileModel> NewLevelModel = StaticCastSharedPtr<FWorldTileModel>(CreateNewEmptyLevel());
	
	if (NewLevelModel.IsValid())
	{
		// Load it 
		NewLevelModel->LoadLevel();

		FLevelModelList Levels; 
		Levels.Add(NewLevelModel);

		ALandscapeProxy* SourceLandscape = LandscapeTileModel->GetLandscape();
	
		NewLevelModel->SetVisibleInEditor(false);
		NewLevelModel->CreateAdjacentLandscapeProxy(SourceLandscape, InWhere);
		ShowLevelsInEditor(Levels);
	}
}

static bool ReadHeightmapFile(TArray<uint16>& Result, const FString& Filename, int32 ExpectedWidth, int32 ExpectedHeight)
{
	bool bResult = true;

	ILandscapeEditorModule& LandscapeEditorModule = FModuleManager::GetModuleChecked<ILandscapeEditorModule>("LandscapeEditor");
	const ILandscapeHeightmapFileFormat* HeightmapFormat = LandscapeEditorModule.GetHeightmapFormatByExtension(*FPaths::GetExtension(Filename, true));

	FLandscapeHeightmapImportData ImportData = HeightmapFormat->Import(*Filename, NAME_None, {(uint32)ExpectedWidth, (uint32)ExpectedHeight});
	if (ImportData.ResultCode != ELandscapeImportResult::Error)
	{
		Result = MoveTemp(ImportData.Data);
	}
	else
	{
		UE_LOGF(LogStreaming, Warning, "%ls", *ImportData.ErrorMessage.ToString());
		Result.Empty();
		bResult = false;
	}

	return bResult;
}

static bool ReadWeightmapFile(TArray<uint8>& Result, const FString& Filename, FName LayerName, int32 ExpectedWidth, int32 ExpectedHeight)
{
	bool bResult = true;

	ILandscapeEditorModule& LandscapeEditorModule = FModuleManager::GetModuleChecked<ILandscapeEditorModule>("LandscapeEditor");
	const ILandscapeWeightmapFileFormat* WeightmapFormat = LandscapeEditorModule.GetWeightmapFormatByExtension(*FPaths::GetExtension(Filename, true));

	FLandscapeWeightmapImportData ImportData = WeightmapFormat->Import(*Filename, LayerName, {(uint32)ExpectedWidth, (uint32)ExpectedHeight});
	if (ImportData.ResultCode != ELandscapeImportResult::Error)
	{
		Result = MoveTemp(ImportData.Data);
	}
	else
	{
		UE_LOGF(LogStreaming, Warning, "%ls", *ImportData.ErrorMessage.ToString());
		Result.Empty();
		bResult = false;
	}

	return bResult;
}

static ULandscapeLayerInfoObject* GetLandscapeLayerInfoObject(FName LayerName, const FString& ContentPath)
{
	// Build default layer object name and package name
	FString LayerObjectName = FString::Printf(TEXT("%s_LayerInfo"), *LayerName.ToString());
	FString Path = ContentPath + TEXT("_sharedassets/");
	if (Path.StartsWith("/Temp/"))
	{
		Path = FString("/Game/") + Path.RightChop(FString("/Temp/").Len());
	}
	
	FString PackageName = Path + LayerObjectName;
	UPackage* Package = FindPackage(nullptr, *PackageName);
	if (Package == nullptr)
	{
		Package = CreatePackage( *PackageName);
	}

	ULandscapeLayerInfoObject* LayerInfo = FindObject<ULandscapeLayerInfoObject>(Package, *LayerObjectName);
	if (LayerInfo == nullptr)
	{
		// Creates a new layer info object, using the default if available, or a new empty one
		LayerInfo = UE::Landscape::CreateTargetLayerInfo(LayerName, Path, LayerObjectName);

		TArray<UPackage*> PackagesToSave; 
		PackagesToSave.Add(Package);
		FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, false, false);
	}
		
	return LayerInfo;
}

static void SetupLandscapeImportLayers(const FTiledLandscapeImportSettings& InImportSettings, const FString& ContentPath, int32 TileIndex, TArray<FLandscapeImportLayerInfo>& OutLayerInfo)
{
	for (const auto& LayerSettings : InImportSettings.LandscapeLayerSettingsList)
	{
		FLandscapeImportLayerInfo LayerImportInfo(LayerSettings.Name);
		
		// Do we have a weightmap data for this tile?
		const FString* WeightmapFile = nullptr;
		if (InImportSettings.TileCoordinates.IsValidIndex(TileIndex))
		{
			FIntPoint TileCoordinates = InImportSettings.TileCoordinates[TileIndex];
			WeightmapFile = LayerSettings.WeightmapFiles.Find(TileCoordinates);
			if (WeightmapFile)
			{
				LayerImportInfo.SourceFilePath = *WeightmapFile;
				ReadWeightmapFile(LayerImportInfo.LayerData, LayerImportInfo.SourceFilePath, LayerImportInfo.LayerName, InImportSettings.SizeX, InImportSettings.SizeX);
			}
		}

		LayerImportInfo.LayerInfo = GetLandscapeLayerInfoObject(LayerImportInfo.LayerName, ContentPath);
		LayerImportInfo.LayerInfo->SetBlendMethod(LayerSettings.bNoBlendWeight ? ELandscapeTargetLayerBlendMethod::None : ELandscapeTargetLayerBlendMethod::FinalWeightBlending, /*bInModify = */true);

		OutLayerInfo.Add(LayerImportInfo);
	}
}

void FWorldTileCollectionModel::ImportTiledLandscape_Executed()
{
	/** Create the window to host widget */
	TSharedRef<SWindow> ImportWindow = SNew(SWindow)
											.Title(LOCTEXT("TiledLandcapeImport_DialogTitle", "Import Tiled Landscape"))
											.SizingRule( ESizingRule::Autosized )
											.SupportsMinimize(false) 
											.SupportsMaximize(false);

	/** Set the content of the window */
	TSharedRef<STiledLandscapeImportDlg> ImportDialog = SNew(STiledLandscapeImportDlg, ImportWindow);
	ImportWindow->SetContent(ImportDialog);

	/** Show the dialog window as a modal window */
	GEditor->EditorAddModalWindow(ImportWindow);

	if (ImportDialog->ShouldImport() && ImportDialog->GetImportSettings().HeightmapFileList.Num())
	{
		const FTiledLandscapeImportSettings& ImportSettings = ImportDialog->GetImportSettings();

		// Default path for imported landscape tiles
		// Use tile prefix as a folder name under world root
		FString WorldRootPath = FPackageName::LongPackageNameToFilename(GetWorld()->WorldComposition->GetWorldRoot());
		// Extract tile prefix
		FString FolderName = FPaths::GetBaseFilename(ImportSettings.HeightmapFileList[0]);
		int32 PrefixEnd = FolderName.Find(TEXT("_x"), ESearchCase::IgnoreCase, ESearchDir::FromEnd);
		FolderName.LeftInline(PrefixEnd, EAllowShrinking::No);
		WorldRootPath+= FolderName;
		WorldRootPath+= TEXT("/");

		GWarn->BeginSlowTask(LOCTEXT("ImportingLandscapeTilesBegin", "Importing landscape tiles"), true);

		// Create main landscape actor in persistent level, it will be empty (no components in it)
		// All landscape tiles will go into it's own sub-levels
		FGuid LandscapeGuid = FGuid::NewGuid();
		{
			ALandscape* Landscape = GetWorld()->SpawnActor<ALandscape>();
			Landscape->SetActorTransform(FTransform(FQuat::Identity, FVector::ZeroVector, ImportSettings.Scale3D));

			// Setup layers list for importing
			TArray<FLandscapeImportLayerInfo> ImportLayers;
			SetupLandscapeImportLayers(ImportSettings, GetWorld()->GetOutermost()->GetName(), INDEX_NONE, ImportLayers);

			// Set landscape configuration
			Landscape->LandscapeMaterial	= ImportSettings.LandscapeMaterial.Get();
			Landscape->ComponentSizeQuads	= ImportSettings.QuadsPerSection*ImportSettings.SectionsPerComponent;
			Landscape->NumSubsections		= ImportSettings.SectionsPerComponent;
			Landscape->SubsectionSizeQuads	= ImportSettings.QuadsPerSection;
			Landscape->SetLandscapeGuid(LandscapeGuid);
			for (const auto& ImportLayerInfo : ImportLayers)
			{
				Landscape->AddTargetLayer(ImportLayerInfo.LayerName, FLandscapeTargetLayerSettings(ImportLayerInfo.LayerInfo));
			}
			Landscape->CreateLandscapeInfo();
		}

		// Import tiles
		int32 TileIndex = 0;
		for (const FString& Filename : ImportSettings.HeightmapFileList)
		{
			check(LandscapeGuid.IsValid());

			FString TileName = FPaths::GetBaseFilename(Filename);
			FVector TileScale = ImportSettings.Scale3D;

			GWarn->StatusUpdate(TileIndex, ImportSettings.HeightmapFileList.Num(), FText::Format(LOCTEXT("ImportingLandscapeTiles", "Importing landscape tiles: {0}"), FText::FromString(TileName)));

			FWorldTileModel::FLandscapeImportSettings TileImportSettings = {};
			TileImportSettings.LandscapeGuid		= LandscapeGuid;
			TileImportSettings.LandscapeMaterial	= ImportSettings.LandscapeMaterial.Get();
			TileImportSettings.ComponentSizeQuads	= ImportSettings.QuadsPerSection*ImportSettings.SectionsPerComponent;
			TileImportSettings.QuadsPerSection		= ImportSettings.QuadsPerSection;
			TileImportSettings.SectionsPerComponent = ImportSettings.SectionsPerComponent;
			TileImportSettings.SizeX				= ImportSettings.SizeX;
			TileImportSettings.SizeY				= ImportSettings.SizeX;
			TileImportSettings.HeightmapFilename	= Filename;
			TileImportSettings.LandscapeTransform.SetScale3D(TileScale);

			// Setup layers list for importing
			SetupLandscapeImportLayers(ImportSettings, GetWorld()->GetOutermost()->GetName(), TileIndex, TileImportSettings.ImportLayers);
			TileImportSettings.ImportLayerType = ELandscapeImportAlphamapType::Additive;

			if (ReadHeightmapFile(TileImportSettings.HeightData, Filename, TileImportSettings.SizeX, TileImportSettings.SizeY))
			{
				FString MapFileName = WorldRootPath + TileName + FPackageName::GetMapPackageExtension();
				// Create a new world - so we can 'borrow' its level
				UWorld* NewWorld = UWorld::CreateWorld(EWorldType::None, false);
				check(NewWorld);

				bool bSaved = FEditorFileUtils::SaveLevel(NewWorld->PersistentLevel, *MapFileName);
				if (bSaved)
				{
					// update levels list so we can find a new level in our world model
					PopulateLevelsList();
					TSharedPtr<FWorldTileModel> NewTileModel = StaticCastSharedPtr<FWorldTileModel>(FindLevelModel(NewWorld->GetOutermost()->GetFName()));
					// Hide level, so we do not depend on a current world origin
					NewTileModel->SetVisibleInEditor(false);

					// Create landscape proxy in a new level
					ALandscapeProxy* NewLandscape = NewTileModel->ImportLandscapeTile(TileImportSettings);

					if (NewLandscape)
					{
						// Set bounds of a tile
						NewTileModel->TileDetails->Bounds = NewLandscape->GetComponentsBoundingBox();

						// Calculate this tile offset from world origin
						FIntRect NewLandscapeRect = NewLandscape->GetBoundingRect();
						float WidthX = NewLandscapeRect.Width()*TileScale.X;
						float WidthY = NewLandscapeRect.Height()*TileScale.Y;
						FIntPoint TileCoordinates = ImportSettings.TileCoordinates[TileIndex] + ImportSettings.TilesCoordinatesOffset;
						FIntPoint TileOffset = FIntPoint(TileCoordinates.X*WidthX, TileCoordinates.Y*WidthY);
						if (ImportSettings.bFlipYAxis)
						{
							TileOffset.Y = -(TileOffset.Y + WidthY);
						}

						// Place level tile at correct position in the world
						NewTileModel->SetLevelPosition(FIntVector(TileOffset.X, TileOffset.Y, 0));

						// Save level with a landscape
						FEditorFileUtils::SaveLevel(NewWorld->PersistentLevel, *MapFileName);
					}

					// Destroy the new world we created and collect the garbage
					NewWorld->DestroyWorld(false);
					CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
				}
			}

			TileIndex++;
		}

		GWarn->EndSlowTask();
	}
}

void FWorldTileCollectionModel::ReimportTiledLandscape_Executed(FName TargetLayer)
{
	// Collect selected landscape tiles
	TArray<TSharedPtr<FWorldTileModel>> TargetLandscapeTiles;
	for (auto LevelModel : SelectedLevelsList)
	{
		TSharedPtr<FWorldTileModel> TileModel = StaticCastSharedPtr<FWorldTileModel>(LevelModel);
		if (TileModel->IsEditable() && TileModel->CanReimportHeightmap())
		{
			TargetLandscapeTiles.Add(TileModel);
		}
	}

	if (TargetLandscapeTiles.Num() == 0)
	{
		return;
	}

	FScopedSlowTask Progress(TargetLandscapeTiles.Num(), LOCTEXT("LandscapeImportProgress", "Reimporting landscape tiles..."));
	Progress.MakeDialog();

	TArray<bool> AllLevelsVisibilityState;
	// Hide all visible levels
	for (auto LevelModel : AllLevelsList)
	{
		AllLevelsVisibilityState.Add(LevelModel->IsVisibleInEditor());
		if (!LevelModel->IsPersistent())
		{
			LevelModel->SetVisibleInEditor(false);
		}
	}

	// Disable world origin tracking, so we can show, hide levels without offseting them
	GetWorld()->WorldComposition->bTemporarilyDisableOriginTracking = true;

	// Reimport data for each selected landscape tile
	for (auto TileModel : TargetLandscapeTiles)
	{
		Progress.EnterProgressFrame();

		TileModel->SetVisibleInEditor(true);

		ALandscapeProxy* Landscape = TileModel->GetLandscape();
		FIntRect LandscapeSize = Landscape->GetBoundingRect();
		int32 NumSamplesX = LandscapeSize.Width() + 1;
		int32 NumSamplesY = LandscapeSize.Height() + 1;

		ULandscapeLayerInfoObject* DataLayer = ALandscapeProxy::VisibilityLayer;

		if (TargetLayer == HeightmapLayerName) // Heightmap
		{
			if (!Landscape->ReimportHeightmapFilePath.IsEmpty())
			{
				TArray<uint16> RawData;
				ReadHeightmapFile(RawData, *Landscape->ReimportHeightmapFilePath, NumSamplesX, NumSamplesY);
				FScopedSetLandscapeEditingLayer Scope(Landscape->GetLandscapeActor(), Landscape->ReimportDestinationLayerGuid);
				LandscapeEditorUtils::SetHeightmapData(Landscape, RawData);
			}
		}
		else // Weightmap
		{
			for (const TPair<FName, FLandscapeTargetLayerSettings>& Pair : Landscape->GetTargetLayers())
			{
				const FLandscapeTargetLayerSettings& LayerSettings = Pair.Value;
				if (LayerSettings.LayerInfoObj && (LayerSettings.LayerInfoObj->GetLayerName() == TargetLayer || TargetLayer == NAME_None))
				{
					if (!LayerSettings.ReimportLayerFilePath.IsEmpty())
					{
						TArray<uint8> RawData;
						ReadWeightmapFile(RawData, *LayerSettings.ReimportLayerFilePath, LayerSettings.LayerInfoObj->GetLayerName(), NumSamplesX, NumSamplesY);
						FScopedSetLandscapeEditingLayer Scope(Landscape->GetLandscapeActor(), Landscape->ReimportDestinationLayerGuid);
						LandscapeEditorUtils::SetWeightmapData(Landscape, LayerSettings.LayerInfoObj, RawData);

						if (TargetLayer != NAME_None)
						{
							// Importing one specific layer
							break;
						}
					}
				}
			}
		}

		TileModel->SetVisibleInEditor(false);
	}

	// Restore world origin tracking
	GetWorld()->WorldComposition->bTemporarilyDisableOriginTracking = false;

	// Restore levels visibility
	for (int32 LevelIdx = 0; LevelIdx < AllLevelsList.Num(); ++LevelIdx)
	{
		if (AllLevelsVisibilityState[LevelIdx])
		{
			AllLevelsList[LevelIdx]->SetVisibleInEditor(true);
		}
	}
}

void FWorldTileCollectionModel::OnToggleLockTilesLocation()
{
	bool bEnabled = GetWorld()->WorldComposition->bLockTilesLocation;
	GetWorld()->WorldComposition->bLockTilesLocation = !bEnabled;
}

bool FWorldTileCollectionModel::IsLockTilesLocationEnabled()
{
	return GetWorld()->WorldComposition->bLockTilesLocation;
}

void FWorldTileCollectionModel::PostUndo(bool bSuccess)
{
	if (!bIsSavingLevel)
	{
		RequestUpdateAllLevels();
	}
}

void FWorldTileCollectionModel::MoveWorldOrigin(const FIntPoint& InOrigin2D)
{
	if (IsReadOnly())
	{
		return;
	}
	
	GetWorld()->SetNewWorldOrigin(FIntVector(InOrigin2D.X, InOrigin2D.Y, 0));
	RequestUpdateAllLevels();
}

void FWorldTileCollectionModel::MoveWorldOrigin_Executed()
{
	if (!IsOneLevelSelected() || !IsOriginRebasingEnabled())
	{
		return;
	}

	TSharedPtr<FWorldTileModel> TargetModel = StaticCastSharedPtr<FWorldTileModel>(SelectedLevelsList[0]);
	FIntVector IntOrigin = FIntVector(TargetModel->GetAbsoluteLevelPosition());
	FIntPoint IntOrign2D = FIntPoint(IntOrigin.X, IntOrigin.Y);
	MoveWorldOrigin(IntOrign2D);
}

void FWorldTileCollectionModel::ResetWorldOrigin_Executed()
{
	if (IsOriginRebasingEnabled())
	{
		FBox OriginArea = EditableWorldArea().ShiftBy(FVector(GetWorld()->OriginLocation));
		Focus(OriginArea, OriginAtCenter);
	
		MoveWorldOrigin(FIntPoint::ZeroValue);
	}
}

void FWorldTileCollectionModel::ResetLevelOrigin_Executed()
{
	if (IsReadOnly())
	{
		return;
	}
	
	for(auto It = SelectedLevelsList.CreateConstIterator(); It; ++It)
	{
		TSharedPtr<FWorldTileModel> TileModel = StaticCastSharedPtr<FWorldTileModel>(*It);
		
		FIntVector AbsolutePosition = TileModel->GetAbsoluteLevelPosition();
		FIntPoint AbsolutePosition2D = FIntPoint(AbsolutePosition.X, AbsolutePosition.Y);
		if (AbsolutePosition2D != FIntPoint::ZeroValue)
		{
			FLevelModelList LevelsToMove; LevelsToMove.Add(TileModel);
			TranslateLevels(LevelsToMove, FIntPoint::ZeroValue - AbsolutePosition2D, false);
		}
	}

	RequestUpdateAllLevels();
}

void FWorldTileCollectionModel::OnPreSaveWorld(UWorld* World, FObjectPreSaveContext ObjectSaveContext)
{
	// Levels during OnSave procedure might be moved to original position
	// and then back to position with offset
	bIsSavingLevel = true;
}

void FWorldTileCollectionModel::OnPostSaveWorld(UWorld* World, FObjectPostSaveContext ObjectSaveContext)
{
	bIsSavingLevel = false;
}

void FWorldTileCollectionModel::OnNewCurrentLevel()
{
	TSharedPtr<FLevelModel> CurrentLevelModel = FindLevelModel(CurrentWorld->GetCurrentLevel());
	// it's possible the level model has not been registered yet because the RefreshLevelBrowser event hasn't been broadcasted yet : 
	if (CurrentLevelModel.IsValid())
	{
		// Make sure level will be in focus
		Focus(CurrentLevelModel->GetLevelBounds(), FWorldTileCollectionModel::OriginAtCenter);
	}
}

bool FWorldTileCollectionModel::HasMeshProxySupport() const
{
	return bMeshProxyAvailable;
}

bool FWorldTileCollectionModel::GenerateLODLevels(FLevelModelList InLevelList, int32 TargetLODIndex)
{
	IMeshReductionModule& ReductionModule = FModuleManager::Get().LoadModuleChecked<IMeshReductionModule>("MeshReductionInterface");

	// Select tiles that can be processed
	TArray<TSharedPtr<FWorldTileModel>> TilesToProcess;
	TArray<FString>						LODPackageNames;
	for (TSharedPtr<FLevelModel> LevelModel : InLevelList)
	{
		ULevel* SourceLevel = LevelModel->GetLevelObject();
		if (SourceLevel == nullptr)
		{
			continue;
		}

		TSharedPtr<FWorldTileModel> TileModel = StaticCastSharedPtr<FWorldTileModel>(LevelModel);
		FWorldTileInfo TileInfo = TileModel->TileDetails->GetInfo();
		if (!TileInfo.LODList.IsValidIndex(TargetLODIndex))
		{
			continue;
		}

		TilesToProcess.Add(TileModel);
	}

	// TODO: Need to check out all LOD maps here
	
	GWarn->BeginSlowTask(LOCTEXT("GenerateLODLevel", "Generating LOD Levels"), true);
	// Generate LOD maps for each tile
	for (TSharedPtr<FWorldTileModel> TileModel : TilesToProcess)
	{
		TArray<AActor*>				Actors;
		TArray<ALandscapeProxy*>	LandscapeActors;
		// Separate landscape actors from all others
		for (AActor* Actor : TileModel->GetLevelObject()->Actors)
		{
			if (Actor)
			{
				ALandscapeProxy* LandscapeProxy = Cast<ALandscapeProxy>(Actor);
				if (LandscapeProxy)
				{
					LandscapeActors.Add(LandscapeProxy);
				}
				else if(!Actor->IsA<AInstancedFoliageActor>()) // Exclude Foliage from merged mesh
				{
					Actors.Add(Actor);
				}
			}
		}

		// Check if we can simplify this level
		IMeshMerging* MeshMerging = ReductionModule.GetMeshMergingInterface();
		if (MeshMerging == nullptr && LandscapeActors.Num() == 0)
		{
			continue;
		}

		// We have to make original level visible, to correctly export it
		const bool bVisibleLevel = TileModel->IsVisibleInEditor();
		if (!bVisibleLevel)
		{
			GetWorld()->WorldComposition->bTemporarilyDisableOriginTracking = true;
			TileModel->SetVisibleInEditor(true);
		}

		FLevelSimplificationDetails SimplificationDetails = TileModel->GetLevelObject()->LevelSimplification[TargetLODIndex];
		
		// Source level package name
		FString SourceLongPackageName = TileModel->TileDetails->PackageName.ToString();
		FString SourceShortPackageName = FPackageName::GetShortName(SourceLongPackageName);
		// Target PackageName for generated level: /LongPackageName+LOD/ShortPackageName_LOD[LODIndex]
		const FString LODLevelPackageName = FString::Printf(TEXT("%sLOD/%s_LOD%d"),	*SourceLongPackageName, *SourceShortPackageName, TargetLODIndex+1);

		// Create a package for a LOD level
		UPackage* LODPackage = CreatePackage( *LODLevelPackageName);
		LODPackage->FullyLoad();
		LODPackage->Modify();
		// This is a hack to avoid save file dialog when we will be saving LOD map package
		FPackagePath LODLevelPackagePath = FPackagePath::FromPackageNameChecked(LODLevelPackageName);
		LODLevelPackagePath.SetHeaderExtension(EPackageExtension::Map);
		LODPackage->SetLoadedPath(LODLevelPackagePath);

		// This is current actors offset from their original position
		FVector ActorsOffset = FVector(TileModel->GetAbsoluteLevelPosition() - GetWorld()->OriginLocation);
		if (GetWorld()->WorldComposition->bTemporarilyDisableOriginTracking)
		{
			ActorsOffset = FVector::ZeroVector;
		}
	
		struct FAssetToSpawnInfo
		{
			FAssetToSpawnInfo(UStaticMesh* InStaticMesh, const FTransform& InTransform, ALandscapeProxy* InSourceLandscape = nullptr, int32 InLandscapeLOD = 0)
			: StaticMesh(InStaticMesh)
			, Transform(InTransform)
			, SourceLandscape(InSourceLandscape)
			, LandscapeLOD(InLandscapeLOD)
			{}

			UStaticMesh* StaticMesh;
			FTransform Transform;
			ALandscapeProxy* SourceLandscape;
			int32 LandscapeLOD;
		};

		TArray<FAssetToSpawnInfo>	AssetsToSpawn;
		TArray<UObject*>			GeneratedAssets;

		// Where generated assets will be stored
		UPackage* AssetsOuter = SimplificationDetails.bCreatePackagePerAsset ? nullptr : LODPackage;
		// In case we don't have outer generated assets should have same path as LOD level
		const FString AssetsPath = SimplificationDetails.bCreatePackagePerAsset ? FPackageName::GetLongPackagePath(LODLevelPackageName) + TEXT("/") : TEXT("");
	
		// Generate Proxy LOD mesh for all actors excluding landscapes
		if (Actors.Num() && MeshMerging != nullptr)
		{
			GWarn->StatusUpdate(0, 10, LOCTEXT("GeneratingProxyMesh", "Generating Proxy Mesh"));

			FMeshProxySettings ProxySettings;
			ProxySettings.ScreenSize = ProxySettings.ScreenSize*(SimplificationDetails.DetailsPercentage/100.f);
			ProxySettings.MaterialSettings = SimplificationDetails.StaticMeshMaterialSettings;

			FString ProxyPackageName = FString::Printf(TEXT("PROXY_%s_LOD%d"), *FPackageName::GetShortName(TileModel->TileDetails->PackageName), TargetLODIndex + 1);
			
			// Generate proxy mesh and proxy material assets 
			FCreateProxyDelegate ProxyDelegate;
			ProxyDelegate.BindLambda(
				[&](const FGuid Guid, TArray<UObject*>& AssetsToSync)
			{
				//Update the asset registry that a new static mash and material has been created
				if (AssetsToSync.Num())
				{
					UStaticMesh* ProxyMesh = nullptr;
					if (AssetsToSync.FindItemByClass(&ProxyMesh))
					{
						new(AssetsToSpawn)FAssetToSpawnInfo(ProxyMesh, FTransform(-ActorsOffset));
					}

					GeneratedAssets.Append(AssetsToSync);
				}
			});

			FGuid JobGuid = FGuid::NewGuid();

			const IMeshMergeUtilities& MergeUtilities = FModuleManager::Get().LoadModuleChecked<IMeshMergeModule>("MeshMergeUtilities").GetUtilities();
			MergeUtilities.CreateProxyMesh(Actors, ProxySettings, AssetsOuter, AssetsPath + ProxyPackageName, JobGuid, ProxyDelegate);
		}

		// Convert landscape actors into static meshes
		int32 LandscapeActorIndex = 0;
		for (ALandscapeProxy* Landscape : LandscapeActors)
		{
			GWarn->StatusUpdate(LandscapeActorIndex, LandscapeActors.Num(), LOCTEXT("ExportingLandscape", "Exporting Landscape Actors"));

			FFlattenMaterial LandscapeFlattenMaterial;
			FVector LandscapeWorldLocation = Landscape->GetActorLocation();

			int32 LandscapeLOD = SimplificationDetails.LandscapeExportLOD;
			if (!SimplificationDetails.bOverrideLandscapeExportLOD)
			{
				LandscapeLOD = Landscape->MaxLODLevel >= 0 ? Landscape->MaxLODLevel : FMath::CeilLogTwo(Landscape->SubsectionSizeQuads + 1) - 1;
			}

			// Filter out primitives for landscape texture flattening
			TSet<FPrimitiveComponentId> PrimitivesToHide;
			for (TObjectIterator<UPrimitiveComponent> It; It; ++It)
			{
				UPrimitiveComponent* PrimitiveComp = *It;
				UObject* PrimitiveOuter = PrimitiveComp->GetOuter();
				
				const bool bTargetPrim = 
					(PrimitiveComp->GetOuter() == Landscape && !(!SimplificationDetails.bBakeGrassToLandscape && PrimitiveComp->IsA(UHierarchicalInstancedStaticMeshComponent::StaticClass()))) || 
					(SimplificationDetails.bBakeFoliageToLandscape && PrimitiveOuter->IsA(AInstancedFoliageActor::StaticClass()));

				if (!bTargetPrim && PrimitiveComp->IsRegistered() && PrimitiveComp->SceneProxy)
				{
					PrimitivesToHide.Add(PrimitiveComp->SceneProxy->GetPrimitiveComponentId());
				}
			}

			if (SimplificationDetails.bBakeGrassToLandscape)
			{
				/* Flush existing grass components, but not grass maps */
				Landscape->FlushGrassComponents(nullptr, false);
				TArray<FVector> Cameras;
				int32 NumCompsCreated = 0;
				Landscape->UpdateGrass(Cameras, NumCompsCreated, true);
			}
								
			// This is texture resolution for a landscape mesh, probably needs to be calculated using landscape size
			LandscapeFlattenMaterial.SetPropertySize(EFlattenMaterialProperties::Diffuse, SimplificationDetails.LandscapeMaterialSettings.TextureSize);
			LandscapeFlattenMaterial.SetPropertySize(EFlattenMaterialProperties::Normal, SimplificationDetails.LandscapeMaterialSettings.bNormalMap ? SimplificationDetails.LandscapeMaterialSettings.TextureSize : FIntPoint::ZeroValue);
			LandscapeFlattenMaterial.SetPropertySize(EFlattenMaterialProperties::Metallic, SimplificationDetails.LandscapeMaterialSettings.bMetallicMap ? SimplificationDetails.LandscapeMaterialSettings.TextureSize : FIntPoint::ZeroValue);  
			LandscapeFlattenMaterial.SetPropertySize(EFlattenMaterialProperties::Roughness, SimplificationDetails.LandscapeMaterialSettings.bRoughnessMap ? SimplificationDetails.LandscapeMaterialSettings.TextureSize : FIntPoint::ZeroValue);  
			LandscapeFlattenMaterial.SetPropertySize(EFlattenMaterialProperties::Specular, SimplificationDetails.LandscapeMaterialSettings.bSpecularMap ? SimplificationDetails.LandscapeMaterialSettings.TextureSize : FIntPoint::ZeroValue);
			
			FMaterialUtilities::ExportLandscapeMaterial(Landscape, PrimitivesToHide, LandscapeFlattenMaterial);
					
			if (SimplificationDetails.bBakeGrassToLandscape)
			{
				Landscape->FlushGrassComponents(); // wipe this and let it fix itself later
			}
			FString LandscapeBaseAssetName = FString::Printf(TEXT("%s_LOD%d"), *Landscape->GetName(), TargetLODIndex + 1);
			// Construct landscape material
			UMaterial* StaticLandscapeMaterial = FMaterialUtilities::CreateMaterial(
				LandscapeFlattenMaterial, AssetsOuter, *(AssetsPath + LandscapeBaseAssetName), RF_Public | RF_Standalone, SimplificationDetails.LandscapeMaterialSettings, GeneratedAssets);
			// Currently landscape exports world space normal map
			StaticLandscapeMaterial->bTangentSpaceNormal = false;
			StaticLandscapeMaterial->PostEditChange();
	
			// Construct landscape static mesh
			FString LandscapeMeshAssetName = TEXT("SM_") + LandscapeBaseAssetName;
			UPackage* MeshOuter = AssetsOuter;
			if (SimplificationDetails.bCreatePackagePerAsset)
			{
				MeshOuter = CreatePackage( *(AssetsPath + LandscapeMeshAssetName));
				MeshOuter->FullyLoad();
				MeshOuter->Modify();
			}

			auto StaticMesh = NewObject<UStaticMesh>(MeshOuter, *LandscapeMeshAssetName, RF_Public | RF_Standalone);
			{
				StaticMesh->InitResources();
				// make sure it has a new lighting guid
				StaticMesh->SetLightingGuid();

				// Set it to use textured lightmaps. Note that Build Lighting will do the error-checking (texcoordindex exists for all LODs, etc).
				StaticMesh->SetLightMapResolution(64);
				StaticMesh->SetLightMapCoordinateIndex(1);

				FStaticMeshSourceModel& SrcModel = StaticMesh->AddSourceModel();
				/*Don't allow the engine to recalculate normals*/
				SrcModel.BuildSettings.bRecomputeNormals = false;
				SrcModel.BuildSettings.bRecomputeTangents = false;
				SrcModel.BuildSettings.bRemoveDegenerates = false;
				SrcModel.BuildSettings.bUseHighPrecisionTangentBasis = false;
				SrcModel.BuildSettings.bUseFullPrecisionUVs = false;

				//Assign the proxy material to the static mesh
				StaticMesh->GetStaticMaterials().Add(FStaticMaterial(StaticLandscapeMaterial));

				//Set the Imported version before calling the build
				StaticMesh->SetImportVersion(EImportStaticMeshVersion::LastVersion);
			}
			FMeshDescription* LandscapeRawMesh = StaticMesh->CreateMeshDescription(0);
			FStaticMeshAttributes Attributes(*LandscapeRawMesh);
		
			ALandscapeProxy::FRawMeshExportParams ExportParams;
			ExportParams.ExportLOD = LandscapeLOD;
			Landscape->ExportToRawMesh(ExportParams, *LandscapeRawMesh);
		
			TVertexAttributesRef<FVector3f> VertexPositions = Attributes.GetVertexPositions();
			for (const FVertexID VertexID : LandscapeRawMesh->Vertices().GetElementIDs())
			{
				VertexPositions[VertexID] -= (FVector3f)LandscapeWorldLocation;
			}

			//Commit raw mesh and build the staticmesh
			{
				StaticMesh->CommitMeshDescription(0);
				StaticMesh->Build();
				StaticMesh->PostEditChange();
			}

			GeneratedAssets.Add(StaticMesh);
			new(AssetsToSpawn) FAssetToSpawnInfo(StaticMesh, FTransform(LandscapeWorldLocation - ActorsOffset), Landscape, LandscapeLOD);

			LandscapeActorIndex++;
		}

		// Restore level original visibility
		if (!bVisibleLevel)
		{
			TileModel->SetVisibleInEditor(false);
			GetWorld()->WorldComposition->bTemporarilyDisableOriginTracking = false;
		}
	
		if (AssetsToSpawn.Num())
		{
			// Save generated assets
			if (SimplificationDetails.bCreatePackagePerAsset && GeneratedAssets.Num())
			{							
				const bool bCheckDirty = false;
				const bool bPromptToSave = false;
				TArray<UPackage*> PackagesToSave;

				for (UObject* Asset : GeneratedAssets)
				{
					FAssetRegistryModule::AssetCreated(Asset);
					GEditor->BroadcastObjectReimported(Asset);
					PackagesToSave.Add(Asset->GetOutermost());
				}
								
				FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, bCheckDirty, bPromptToSave);

				// Also notify the content browser that the new assets exists
				//FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
				//ContentBrowserModule.Get().SyncBrowserToAssets(GeneratedAssets, true);
			}
			
			// Create new level and spawn generated assets in it
			UWorld* LODWorld = UWorld::FindWorldInPackage(LODPackage);
			if (LODWorld)
			{
				LODWorld->ClearFlags(RF_Public | RF_Standalone);
				LODWorld->DestroyWorld(false);
				LODWorld->Rename(nullptr, GetTransientPackage());
			}
			
			// Create a new world
			LODWorld = UWorld::CreateWorld(EWorldType::None, false, FPackageName::GetShortFName(LODPackage->GetFName()), LODPackage);
			LODWorld->SetFlags(RF_Public | RF_Standalone);

			for (FAssetToSpawnInfo& AssetInfo : AssetsToSpawn)
			{
				FVector Location = AssetInfo.Transform.GetLocation();
				FRotator Rotation(ForceInit);

				if (AssetInfo.SourceLandscape != nullptr)
				{
					ALandscapeMeshProxyActor* MeshActor = LODWorld->SpawnActor<ALandscapeMeshProxyActor>(Location, Rotation);
					MeshActor->GetLandscapeMeshProxyComponent()->InitializeForLandscape(AssetInfo.SourceLandscape, AssetInfo.LandscapeLOD);
					MeshActor->GetLandscapeMeshProxyComponent()->SetStaticMesh(AssetInfo.StaticMesh);			
					MeshActor->SetActorLabel(AssetInfo.SourceLandscape->GetName());
				}
				else
				{
					AStaticMeshActor* MeshActor = LODWorld->SpawnActor<AStaticMeshActor>(Location, Rotation);
					MeshActor->GetStaticMeshComponent()->SetStaticMesh(AssetInfo.StaticMesh);
					MeshActor->SetActorLabel(AssetInfo.StaticMesh->GetName());
				}
			}
		
			// Save generated level
			if (FEditorFileUtils::PromptToCheckoutLevels(false, LODWorld->PersistentLevel))
			{
				FEditorFileUtils::SaveLevel(LODWorld->PersistentLevel, LODLevelPackagePath.GetLocalFullPath());
				FAssetRegistryModule::AssetCreated(LODWorld);
			}
			
			// Destroy the new world we created and collect the garbage
			LODWorld->ClearFlags(RF_Public | RF_Standalone);
			LODWorld->DestroyWorld(false);
			// Also, make sure to release generated assets
			for (UObject* Asset : GeneratedAssets)
			{
				Asset->ClearFlags(RF_Standalone);
			}
			CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
		}
	}

	// Rescan world root
	PopulateLevelsList();
	GWarn->EndSlowTask();	
	
	return true;
}



#undef LOCTEXT_NAMESPACE
