// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Misc/WorldCompositionUtility.h"
#include "Tiles/WorldTileModel.h"
#include "LevelCollectionModel.h"
#include "EditorUndoClient.h"

class FMenuBuilder;
class IDetailsView;

/** The non-UI solution specific presentation logic for a world composition */
class FWorldTileCollectionModel
	: public FLevelCollectionModel 
	, public FEditorUndoClient
{

public:
	/** FWorldTileCollectionModel destructor */
	virtual ~FWorldTileCollectionModel();

	/**  
	 *	Factory method which creates a new FWorldTileCollectionModel object
	 *
	 *	@param	InEditor		The UEditorEngine to use
	 */
	static TSharedRef<FWorldTileCollectionModel> Create(UWorld* InWorld)
	{
		TSharedRef<FWorldTileCollectionModel> LevelCollectionModel(new FWorldTileCollectionModel());
		LevelCollectionModel->Initialize(InWorld);
		return LevelCollectionModel;
	}
public:
	/** FLevelCollection interface */
	virtual void UnloadLevels(const FLevelModelList& InLevelList) override;
	virtual void TranslateLevels(const FLevelModelList& InList, FVector2D InAbsoluteDelta, bool bSnapDelta = true)  override;
	virtual FVector2D SnapTranslationDelta(const FLevelModelList& InList, FVector2D InTranslationDelta, bool bBoundsSnapping, FVector2D::FReal InSnappingValue) override;
	virtual TSharedPtr<WorldHierarchy::FWorldBrowserDragDropOp> CreateDragDropOp() const override;
	virtual TSharedPtr<WorldHierarchy::FWorldBrowserDragDropOp> CreateDragDropOp(const FLevelModelList& InLevels) const override;
	virtual bool PassesAllFilters(const FLevelModel& InLevelModel) const override;
	virtual void BuildHierarchyMenu(FMenuBuilder& InMenuBuilder, EBuildHierarchyMenuFlags Flags) const override;
	virtual void CustomizeFileMainMenu(FMenuBuilder& InMenuBuilder) const override;
	virtual bool GetPlayerView(FVector& Location, FRotator& Rotation) const override;
	virtual bool GetObserverView(FVector& Location, FRotator& Rotation) const override;
	virtual bool CompareLevelsZOrder(TSharedPtr<FLevelModel> InA, TSharedPtr<FLevelModel> InB) const override;
	virtual void RegisterDetailsCustomization(class FPropertyEditorModule& PropertyModule, TSharedPtr<class IDetailsView> InDetailsView)  override;
	virtual void UnregisterDetailsCustomization(class FPropertyEditorModule& PropertyModule, TSharedPtr<class IDetailsView> InDetailsView) override;
	virtual bool IsTileWorld() const override { return true; };
	/** FLevelCollection interface end */

private:
	/** FTickableEditorObject interface */
	void Tick( float DeltaTime ) override;
	/** FTickableEditorObject interface end */

	/** FLevelCollection interface */
	virtual void Initialize(UWorld* InWorld) override;
	virtual void BindCommands() override;
	virtual void OnLevelsCollectionChanged() override;
	virtual void OnLevelsSelectionChanged() override;
	virtual void OnLevelsHierarchyChanged() override;
	virtual void OnPreLoadLevels(const FLevelModelList& InList) override;
	virtual void OnPreShowLevels(const FLevelModelList& InList) override;
	/** FLevelCollection interface end */
	
public:
	/** @return	Whether world browser has world root opened */
	bool HasWorldRoot() const;

	/** Returns TileModel which is used as root for all tiles */
	TSharedPtr<FWorldTileModel> GetWorldRootModel();

	/** Removes selection from levels which belongs to provided Layer */
	void DeselectLevels(const FWorldTileLayer& InLayer);

	/** @return	whether at least one layer is selected */
	bool AreAnyLayersSelected() const;

	/** Hide a levels from the editor and move them to original position 
	 *	Similar to unloading level, but does not removes it from the memory
	 */
	void ShelveLevels(const FWorldTileModelList& InLevels);

	/** Show a levels in the editor and place them to actual world position */
	void UnshelveLevels(const FWorldTileModelList& InLevels);

	/** Whether any of the currently selected levels have landscape actor */
	bool AreAnySelectedLevelsHaveLandscape() const;
	
	/** Creates a new empty level 
	 *	@return LevelModel of a new empty level
	 */
	TSharedPtr<FLevelModel> CreateNewEmptyLevel();

	/** Returns all layers found in the world */
	TArray<FWorldTileLayer>& GetLayers();

	/** Adds unique runtime layer to the world */
	void AddLayer(const FWorldTileLayer& InLayer);
	
	/** Adds unique managed layer to the world */
	void AddManagedLayer(const FWorldTileLayer& InLayer);

	/** Sets provided layer as selected */
	void SetSelectedLayer(const FWorldTileLayer& InLayer); 
	
	/** Sets provided layers list as selected */
	void SetSelectedLayers(const TArray<FWorldTileLayer>& InLayers); 
	
	/** Toggles provided layer selection */
	void ToggleLayerSelection(const FWorldTileLayer& InLayer);
	
	/** Return whether provides layer is selected or not */
	bool IsLayerSelected(const FWorldTileLayer& InLayer);

	/* Notification that level object was loaded from disk */
	void OnLevelLoadedFromDisk(TSharedPtr<FWorldTileModel> InLevel);

	/**	Notification that "view point" for streaming levels visibility preview was changed */
	void UpdateStreamingPreview(FVector2D InPreviewLocation, bool bEnabled);

	/** Returns list of visible streaming levels for current preview location */
	const TMap<FName, int32>& GetPreviewStreamingLevels() const;
		
	/** Calculates snapped moving delta based on specified landscape tile */
	FVector2D SnapTranslationDeltaLandscape(const TSharedPtr<FWorldTileModel>& LandscapeTile, 
											FVector2D InAbsoluteDelta, 
											float SnappingDistance);
	
	//~ Begin FEditorUndoClient Interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }
	// End of FEditorUndoClient

	enum FocusStrategy 
	{
		OriginAtCenter,			// Unconditionally move current world origin to specified area center
		EnsureEditable,			// May move world origin such that specified area become editable
		EnsureEditableCentered  // May move world origin such that specified area become editable and centered in world
	};
	
	/** 
	 *  Tell the browser that user is focusing on this area in world  
	 *  This may cause world origin shifting and subsequent shelving/unshelving operations
	 */
	void Focus(FBox InArea, FocusStrategy InStaragegy);

	/** 
	 *  Builds context menu for a world composition
	 */
	void BuildWorldCompositionMenu(FMenuBuilder& InMenuBuilder) const;

private:
	FWorldTileCollectionModel();
	
	/** Setups parent->child links between tiles */
	void SetupParentChildLinks();

	/** Called before saving world into package file */
	void OnPreSaveWorld(class UWorld* World, FObjectPreSaveContext ObjectSaveContext);
	
	/** Called right after world was saved into package file */
	void OnPostSaveWorld(class UWorld* World, FObjectPostSaveContext ObjectSaveContext);

	/** Called when world has new current level */
	void OnNewCurrentLevel();

	/** @return shared pointer to FStreamingLevelModel associated with provided ULevel object*/
	TSharedPtr<FWorldTileModel> GetCorrespondingModel(ULevel* InLevel);

	/** Moves world origin closer to levels which are going to be loaded 
	 *  and unloads levels which are far enough from new world origin
	 */
	void PrepareToLoadLevels(FWorldTileModelList& InLevels);
		
	/** Delegate callback: the world origin is going to be moved. */
	void PreWorldOriginOffset(UWorld* InWorld, FIntVector InSrcOrigin, FIntVector InDstOrigin);
	
	/** Delegate callback: the world origin has been moved. */
	void PostWorldOriginOffset(UWorld* InWorld, FIntVector InSrcOrigin, FIntVector InDstOrigin);

	/** Update list layers */
	void PopulateLayersList();

	/** Scrolls world origin to specified position */
	void MoveWorldOrigin(const FIntPoint& InOrigin2D);

	/** Adds a loaded level to the world and makes it visible if possible 
	 *	@return Whether level was added to the world
	 */
	bool AddLevelToTheWorld(const TSharedPtr<FWorldTileModel>& InLevel);
	
	/** Fills Layers sub-menu */
	void FillLayersSubMenu(FMenuBuilder& InMenuBuilder) const;
	
	/** Fills adjacent landscape sub-menu */
	void FillAdjacentLandscapeSubMenu(FMenuBuilder& InMenuBuilder) const;

	/** Fills reimport tiled landscape sub-menu */
	void FillReimportTiledLandscapeSubMenu(FMenuBuilder& InMenuBuilder) const;
	
	/** Fills reimport weightmaps sub_menu */
	void FillWeightmapsSubMenu(FMenuBuilder& InMenuBuilder) const;
	
private:
	/** Creates a new level; prompts for level save location */
	void CreateNewLevel_Executed();

	/** Moves world origin to selected level position */
	void MoveWorldOrigin_Executed();
	
	/** Reset world origin offset */
	void ResetWorldOrigin_Executed();

	/** Reset world origin offset */
	void ResetLevelOrigin_Executed();

	/** Clear parent links fro selected levels */
	void ClearParentLink_Executed();

	/** 
	 * Creates a new Level with landscape proxy in it
	 * @param InWhere  Defines on which side of currently selected landscape level
	 */
	void AddLandscapeProxy_Executed(FWorldTileModel::EWorldDirections InWhere);
	
	/** @return whether it is possible to add a level with landscape proxy at specified location */	
	bool CanAddLandscapeProxy(FWorldTileModel::EWorldDirections InWhere) const;

	/** @return Whether selection contains tiles with tiled landscape */	
	bool CanReimportTiledlandscape() const;

	/**  */	
	void ImportTiledLandscape_Executed();

	/**  */	
	void ReimportTiledLandscape_Executed(FName TargetLayer);

	/** Tiles location locking */	
	void OnToggleLockTilesLocation();
public:
	bool IsLockTilesLocationEnabled();

public:
	/** Whether Editor has support for generating static mesh proxies */	
	bool HasMeshProxySupport() const;

	/** 
	 * Generates simplified versions of a specified levels. Levels has to be loaded.
	 * Currently all static meshes found inside one level will be merged into one proxy mesh using Simplygon ProxyLOD
	 * Landscape actors will be converted into static meshes using highest landscape LOD entry
	 */	
	bool GenerateLODLevels(FLevelModelList InLevelList, int32 TargetLODIndex);

	/** Assign selected levels to current layer */
	void AssignSelectedLevelsToLayer_Executed(FWorldTileLayer InLayer);

private:
	/** List of tiles currently not affected by user selection set */
	FLevelModelList						StaticTileList;	

	/** Cached streaming tiles which are potentially visible from specified view point*/
	TMap<FName, int32>					PreviewVisibleTiles;
	
	/** View point location for calculating potentially visible streaming tiles*/
	FVector								PreviewLocation;

	/** All layers */
	TArray<FWorldTileLayer>				AllLayers;

	/** All layers currently created by the user*/
	TArray<FWorldTileLayer>				ManagedLayers;

	/** All selected layers */
	TArray<FWorldTileLayer>				SelectedLayers;

	// Is in process of saving a level
	bool								bIsSavingLevel;
	
	// Whether Editor has support for mesh proxy
	bool								bMeshProxyAvailable;
};
