// Copyright Epic Games, Inc. All Rights Reserved.
#include "Tiles/WorldTileDetails.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Misc/AssertionMacros.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldTileDetails)


#define LOCTEXT_NAMESPACE "WorldBrowser"


/////////////////////////////////////////////////////
// FTileLODEntryDetails
FTileLODEntryDetails::FTileLODEntryDetails()
	// Initialize properties with default values from FWorldTileLODInfo
	: LODIndex(0)
	, RelativeDistance(FWorldTileLODInfo().RelativeStreamingDistance)
{
}

/////////////////////////////////////////////////////
// UWorldTileDetails
UWorldTileDetails::UWorldTileDetails(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	LOD1.LODIndex = 0;
	LOD2.LODIndex = 1;
	LOD3.LODIndex = 2;
	LOD4.LODIndex = 3;
}

void UWorldTileDetails::SetInfo(const FWorldTileInfo& Info, ULevel* Level)
{
	ParentPackageName	= FName(*Info.ParentTilePackageName);
	Position			= Info.Position;
	AbsolutePosition	= Info.AbsolutePosition;
	Layer				= Info.Layer;
	Bounds				= Info.Bounds;
	ZOrder				= Info.ZOrder;
	bHideInTileView		= Info.bHideInTileView;

	// Sync LOD settings
	NumLOD				= Info.LODList.Num();
	FTileLODEntryDetails* LODEntries[WORLDTILE_LOD_MAX_INDEX] = {&LOD1, &LOD2, &LOD3, &LOD4};

	for (int32 i = 0; i < WORLDTILE_LOD_MAX_INDEX; ++i)
	{
		if (Info.LODList.IsValidIndex(i))
		{
			LODEntries[i]->RelativeDistance = Info.LODList[i].RelativeStreamingDistance;
			LODEntries[i]->SimplificationDetails = Level ? Level->LevelSimplification[i] : FLevelSimplificationDetails();
		}
		else
		{
			LODEntries[i]->RelativeDistance = FWorldTileLODInfo().RelativeStreamingDistance;
			LODEntries[i]->SimplificationDetails = FLevelSimplificationDetails();
		}
	}
}
	
FWorldTileInfo UWorldTileDetails::GetInfo() const
{
	FWorldTileInfo Info;
	
	Info.ParentTilePackageName	= ParentPackageName.ToString();
	Info.Position				= Position;
	Info.AbsolutePosition		= AbsolutePosition;
	Info.Layer					= Layer;
	Info.Bounds					= Bounds;
	Info.ZOrder					= ZOrder;
	Info.bHideInTileView		= bHideInTileView;

	// Sync LOD settings
	Info.LODList.SetNum(FMath::Clamp(NumLOD, 0, WORLDTILE_LOD_MAX_INDEX));
	const FTileLODEntryDetails* LODEntries[WORLDTILE_LOD_MAX_INDEX] = {&LOD1, &LOD2, &LOD3, &LOD4};
	for (int32 i = 0; i < Info.LODList.Num(); ++i)
	{
		Info.LODList[i].RelativeStreamingDistance = LODEntries[i]->RelativeDistance;
	}
	
	return Info;
}

void UWorldTileDetails::PostEditChangeProperty( struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = (PropertyChangedEvent.Property != NULL) ? PropertyChangedEvent.Property->GetFName() : NAME_None;
	const FName MemberPropertyName = (PropertyChangedEvent.MemberProperty != NULL) ? PropertyChangedEvent.MemberProperty->GetFName() : NAME_None;

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UWorldTileDetails, Position)
		|| MemberPropertyName == GET_MEMBER_NAME_CHECKED(UWorldTileDetails, Position))
	{
		PositionChangedEvent.Broadcast();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UWorldTileDetails, ParentPackageName))
	{
		ParentPackageNameChangedEvent.Broadcast();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UWorldTileDetails, NumLOD))
	{
		LODSettingsChangedEvent.Broadcast();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UWorldTileDetails, LOD1)
			|| MemberPropertyName == GET_MEMBER_NAME_CHECKED(UWorldTileDetails, LOD1))
	{
		LODSettingsChangedEvent.Broadcast();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UWorldTileDetails, LOD2)
			|| MemberPropertyName == GET_MEMBER_NAME_CHECKED(UWorldTileDetails, LOD2))
	{
		LODSettingsChangedEvent.Broadcast();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UWorldTileDetails, LOD3)
			|| MemberPropertyName == GET_MEMBER_NAME_CHECKED(UWorldTileDetails, LOD3))
	{
		LODSettingsChangedEvent.Broadcast();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UWorldTileDetails, LOD4)
			|| MemberPropertyName == GET_MEMBER_NAME_CHECKED(UWorldTileDetails, LOD4))
	{
		LODSettingsChangedEvent.Broadcast();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UWorldTileDetails, ZOrder))
	{
		ZOrderChangedEvent.Broadcast();
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UWorldTileDetails, bHideInTileView))
	{
		HideInTileViewChangedEvent.Broadcast();
	}
}

void UWorldTileDetails::PostEditUndo()
{
	Super::PostEditUndo();
	
	PostUndoEvent.Broadcast();
}

#undef LOCTEXT_NAMESPACE
