// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Delegates/Delegate.h"
#include "Engine/Level.h"
#include "Math/Box.h"
#include "Math/IntVector.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/WorldCompositionUtility.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/UObjectGlobals.h"

#include "WorldTileDetails.generated.h"

/////////////////////////////////////////////////////
// FTileLODEntryDetails
//
// Helper class to hold tile LOD level description
//
USTRUCT()
struct FTileLODEntryDetails
{
	GENERATED_USTRUCT_BODY()
	
	// Maximum deviation of details percentage
	UPROPERTY(Category=LODDetails, VisibleAnywhere)
	int32		LODIndex;
			
	// Relative to original tile streaming distance (Absolute Distance = Layer Streaming Distance + Relative Distance)
	UPROPERTY(Category=LODDetails, EditAnywhere, meta=(ClampMin = "10", ClampMax = "10000000", UIMin = "10", UIMax = "500000"))
	int32		RelativeDistance;

	UPROPERTY(Category=ReductionSettings, EditAnywhere)	
	FLevelSimplificationDetails SimplificationDetails;

	FTileLODEntryDetails();
};

/////////////////////////////////////////////////////
// UWorldTileDetails
// 
// Helper class to hold world tile information
// Holding this information in UObject gives us ability to use property editors and support undo operations
// 
UCLASS()
class UWorldTileDetails : public UObject
{
	GENERATED_UCLASS_BODY()
	
	// Whether this tile properties can be edited via details panel
	UPROPERTY(Category=Tile, VisibleAnywhere)
	bool							bTileEditable;
		
	// Tile long package name (readonly)	
	UPROPERTY(Category=Tile, VisibleAnywhere)
	FName							PackageName;
	
	// Parent tile long package name	
	UPROPERTY(Category=Tile, EditAnywhere)
	FName							ParentPackageName;
	
	// Tile position in the world, relative to parent 
	UPROPERTY(Category=Tile, EditAnywhere )
	FIntVector						Position;

	// Tile absolute position in the world (readonly)
	UPROPERTY(Category=Tile, VisibleAnywhere)
	FIntVector						AbsolutePosition;

	// Tile sorting order
	UPROPERTY(Category=Tile, EditAnywhere, meta=(ClampMin = "-1000", ClampMax = "1000", UIMin = "-1000", UIMax = "1000"))
	int32							ZOrder;

	// Whether to hide tile in the world composition tile view
	UPROPERTY(Category=Tile, EditAnywhere)
	bool							bHideInTileView;
	
	// LOD entries number
	UPROPERTY(Category=LODSettings, EditAnywhere, meta=(ClampMin = "0", ClampMax = "4", UIMin = "0", UIMax = "4"))
	int32							NumLOD;

	UPROPERTY(Category=LODSettings, EditAnywhere)
	FTileLODEntryDetails			LOD1;

	UPROPERTY(Category=LODSettings, EditAnywhere)
	FTileLODEntryDetails			LOD2;

	UPROPERTY(Category=LODSettings, EditAnywhere)
	FTileLODEntryDetails			LOD3;

	UPROPERTY(Category=LODSettings, EditAnywhere)
	FTileLODEntryDetails			LOD4;
			
	// Tile layer information
	FWorldTileLayer					Layer;
	
	// Tile bounds
	FBox							Bounds;

	// Whether this tile is a persistent level
	bool							bPersistentLevel;

	// Events when user edits tile properties
	DECLARE_EVENT( UWorldTileDetails, FTilePropertyChanged )

	FTilePropertyChanged			PostUndoEvent;
	FTilePropertyChanged			PositionChangedEvent;
	FTilePropertyChanged			ParentPackageNameChangedEvent;
	FTilePropertyChanged			LODSettingsChangedEvent;
	FTilePropertyChanged			ZOrderChangedEvent;
	FTilePropertyChanged			HideInTileViewChangedEvent;
		
	// UObject interface
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostEditUndo() override;
	
public:
	// Initialize tile details with values stored in FWorldTileInfo object
	void SetInfo(const FWorldTileInfo& Info, ULevel* Level);
	
	// Gets the initialized FWorldTileInfo from this details values
	FWorldTileInfo GetInfo() const;

};
