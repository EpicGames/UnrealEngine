// Copyright Epic Games, Inc. All Rights Reserved.
#include "Tiles/WorldTileDetailsCustomization.h"
#include "Misc/PackageName.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"

#include "Engine/WorldComposition.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "DetailCategoryBuilder.h"

#include "SPropertyEditorLevelPackage.h"
#include "Tiles/WorldTileDetails.h"

#define LOCTEXT_NAMESPACE "WorldBrowser"

// Width for level package selection combo box
static const float LevelPackageWidgetMinDesiredWidth = 1000.f;

static FString GetWorldRoot(TWeakPtr<FWorldTileCollectionModel> WorldModel)
{
	auto PinnedWorldData = WorldModel.Pin();
	if (PinnedWorldData.IsValid())
	{
		return PinnedWorldData->GetWorld()->WorldComposition->GetWorldRoot();
	}
	else
	{
		return FPackageName::FilenameToLongPackageName(FPaths::ProjectContentDir());
	}
}

static bool HasLODSuffix(const FString& InPackageName)
{
	return InPackageName.Contains(WORLDTILE_LOD_PACKAGE_SUFFIX, ESearchCase::IgnoreCase, ESearchDir::FromEnd);
}

static bool IsPersistentLevel(const FString& InPackageName, const TSharedPtr<FWorldTileCollectionModel>& InWorldData)
{
	if (InWorldData.IsValid() && InWorldData->GetWorld())
	{
		if (InWorldData->GetWorld()->GetOutermost()->GetName() == InPackageName)
		{
			return true;
		}
	}
	
	return false;
}

/////////////////////////////////////////////////////
// FWorldTileDetails 
TSharedRef<IDetailCustomization> FWorldTileDetailsCustomization::MakeInstance(TSharedRef<FWorldTileCollectionModel> InWorldModel)
{
	TSharedRef<FWorldTileDetailsCustomization> Instance = MakeShareable(new FWorldTileDetailsCustomization());
	Instance->WorldModel = InWorldModel;
	return Instance;
}

void FWorldTileDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayoutBuilder)
{
	// Property editable state
	TAttribute<bool> IsPropertyEnabled = TAttribute<bool>::Create(
		TAttribute<bool>::FGetter::CreateSP(this, &FWorldTileDetailsCustomization::IsPropertyEditable)
		);
	
	IDetailCategoryBuilder& TileCategory = DetailLayoutBuilder.EditCategory("Tile");

	// Set properties state
	{
		// Package Name
		TileCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UWorldTileDetails, PackageName));
	
		// Parent Package Name
		{
			auto ParentPackagePropertyHandle = DetailLayoutBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UWorldTileDetails, ParentPackageName));
			TileCategory.AddProperty(ParentPackagePropertyHandle)
				.IsEnabled(IsPropertyEnabled)
				.CustomWidget()
					.NameContent()
					[
						ParentPackagePropertyHandle->CreatePropertyNameWidget()
					]
					.ValueContent()
						.MinDesiredWidth(LevelPackageWidgetMinDesiredWidth)
					[
						SNew(SPropertyEditorLevelPackage, ParentPackagePropertyHandle)
							.RootPath(GetWorldRoot(WorldModel))
							.SortAlphabetically(true)
							.OnShouldFilterPackage(this, &FWorldTileDetailsCustomization::OnShouldFilterParentPackage)
					];
		}

		// Position
		TileCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UWorldTileDetails, Position))
			.IsEnabled(IsPropertyEnabled);

		// Absolute Position
		TileCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UWorldTileDetails, AbsolutePosition))
			.IsEnabled(false);

		// Z Order
		TileCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UWorldTileDetails, ZOrder))
			.IsEnabled(IsPropertyEnabled);
		
		// Hide in tile view
		TileCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UWorldTileDetails, bHideInTileView))
			.IsEnabled(IsPropertyEnabled);
		
		// bTileEditable (invisible property to control other properties editable state)
		TileEditableHandle = DetailLayoutBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UWorldTileDetails, bTileEditable));
		TileCategory.AddProperty(TileEditableHandle)
			.Visibility(EVisibility::Hidden);
	}
	
	// LOD
	IDetailCategoryBuilder& LODSettingsCategory = DetailLayoutBuilder.EditCategory("LODSettings");
	{
		NumLODHandle = DetailLayoutBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UWorldTileDetails, NumLOD));
		LODSettingsCategory.AddProperty(NumLODHandle)
			.IsEnabled(IsPropertyEnabled);
		
		LODSettingsCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UWorldTileDetails, LOD1))
			.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FWorldTileDetailsCustomization::GetLODPropertyVisibility, 1)));

		LODSettingsCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UWorldTileDetails, LOD2))
			.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FWorldTileDetailsCustomization::GetLODPropertyVisibility, 2)));

		LODSettingsCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UWorldTileDetails, LOD3))
			.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FWorldTileDetailsCustomization::GetLODPropertyVisibility, 3)));

		LODSettingsCategory.AddProperty(GET_MEMBER_NAME_CHECKED(UWorldTileDetails, LOD4))
			.Visibility(TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateSP(this, &FWorldTileDetailsCustomization::GetLODPropertyVisibility, 4)));
	}
}

bool FWorldTileDetailsCustomization::IsPropertyEditable() const
{
	// Properties are editable if at least one tile is editable
	bool bTileEditable = true;
	if (TileEditableHandle->GetValue(bTileEditable) != FPropertyAccess::Fail)
	{
		return bTileEditable;
	}
	
	return false;
}

EVisibility FWorldTileDetailsCustomization::GetLODPropertyVisibility(int LODIndex) const
{
	int32 NumLOD = MAX_int32;
	if (NumLODHandle->GetValue(NumLOD) != FPropertyAccess::Fail)
	{
		return (NumLOD >= LODIndex ? EVisibility::Visible : EVisibility::Hidden);
	}
	
	return EVisibility::Hidden;
}

bool FWorldTileDetailsCustomization::OnShouldFilterParentPackage(const FString& InPackageName)
{
	// Filter out LOD levels and persistent level
	if (HasLODSuffix(InPackageName) || IsPersistentLevel(InPackageName, WorldModel.Pin()))
	{
		return true;
	}
	
	return false;
}

//////////////////////////////////////////////////////////////////////////////////////////////
// FStreamingLevelDetailsCustomization
TSharedRef<IPropertyTypeCustomization> FStreamingLevelDetailsCustomization::MakeInstance(TSharedRef<FWorldTileCollectionModel> InWorldData) 
{
	TSharedRef<FStreamingLevelDetailsCustomization> Instance = MakeShareable(new FStreamingLevelDetailsCustomization());
	Instance->WorldModel = InWorldData;
	return Instance;
}

void FStreamingLevelDetailsCustomization::CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, 
																class FDetailWidgetRow& HeaderRow, 
																IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	HeaderRow
		.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		];
}

void FStreamingLevelDetailsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, 
																	class IDetailChildrenBuilder& ChildBuilder, 
																	IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	TSharedPtr<IPropertyHandle> StreamingModeProperty = StructPropertyHandle->GetChildHandle(TEXT("StreamingMode"));
	TSharedPtr<IPropertyHandle> PackageNameProperty = StructPropertyHandle->GetChildHandle(TEXT("PackageName"));

	ChildBuilder.AddProperty(StreamingModeProperty.ToSharedRef());
	
	ChildBuilder.AddProperty(PackageNameProperty.ToSharedRef())
		.CustomWidget()
				.NameContent()
				[
					PackageNameProperty->CreatePropertyNameWidget()
				]
				.ValueContent()
					.MinDesiredWidth(LevelPackageWidgetMinDesiredWidth)
				[
					SNew(SPropertyEditorLevelPackage, PackageNameProperty)
						.RootPath(GetWorldRoot(WorldModel))
						.SortAlphabetically(true)
						.OnShouldFilterPackage(this, &FStreamingLevelDetailsCustomization::OnShouldFilterStreamingPackage)
				];

}

bool FStreamingLevelDetailsCustomization::OnShouldFilterStreamingPackage(const FString& InPackageName) const
{
	// Filter out LOD levels and persistent level
	if (HasLODSuffix(InPackageName) || IsPersistentLevel(InPackageName, WorldModel.Pin()))
	{
		return true;
	}
	
	return false;
}


//////////////////////////////////////////////////////////////////////////////////////////////
// FTileLODEntryDetailsCustomization
TSharedRef<IPropertyTypeCustomization> FTileLODEntryDetailsCustomization::MakeInstance(TSharedRef<FWorldTileCollectionModel> InWorldData) 
{
	TSharedRef<FTileLODEntryDetailsCustomization> Instance = MakeShareable(new FTileLODEntryDetailsCustomization());
	Instance->WorldModel = InWorldData;
	return Instance;
}

void FTileLODEntryDetailsCustomization::CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, 
																class FDetailWidgetRow& HeaderRow, 
																IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	HeaderRow
		.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew( SBox )
			.HAlign( HAlign_Center )
			[
				SNew(SButton)
				.Text(LOCTEXT("Generate", "Generate"))
				.OnClicked(this, &FTileLODEntryDetailsCustomization::OnGenerateTile)
				.IsEnabled(this, &FTileLODEntryDetailsCustomization::IsGenerateTileEnabled)
				.ToolTipText(LOCTEXT("GenerateLODToolTip", "Creates simplified sub-level by merging geometry into static mesh proxy (requires Simplygon) and exporting landscapes into static meshes"))
			]
		];
}

void FTileLODEntryDetailsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, 
																	class IDetailChildrenBuilder& ChildBuilder, 
																	IPropertyTypeCustomizationUtils& StructCustomizationUtils )
{
	LODIndexHandle = StructPropertyHandle->GetChildHandle(
		GET_MEMBER_NAME_CHECKED(FTileLODEntryDetails, LODIndex)
		);
	
	TSharedPtr<IPropertyHandle> DistanceProperty = StructPropertyHandle->GetChildHandle(
		GET_MEMBER_NAME_CHECKED(FTileLODEntryDetails, RelativeDistance)
		);
	
	TSharedPtr<IPropertyHandle> SimplificationDetails = StructPropertyHandle->GetChildHandle(
		GET_MEMBER_NAME_CHECKED(FTileLODEntryDetails, SimplificationDetails)
		);

	ChildBuilder.AddProperty(LODIndexHandle.ToSharedRef())
		.Visibility(EVisibility::Hidden);

	ChildBuilder.AddProperty(DistanceProperty.ToSharedRef())
		.IsEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &FTileLODEntryDetailsCustomization::IsLODDistanceEnabled)));
	
	ChildBuilder.AddProperty(SimplificationDetails.ToSharedRef())
		.IsEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &FTileLODEntryDetailsCustomization::IsGenerateTileEnabled)));
}

FReply FTileLODEntryDetailsCustomization::OnGenerateTile()
{
	int32 LODIndex;
	if (LODIndexHandle->GetValue(LODIndex) == FPropertyAccess::Success)
	{
		TSharedPtr<FWorldTileCollectionModel> PinnedWorldModel = WorldModel.Pin();
		if (PinnedWorldModel.IsValid())
		{
			FLevelModelList LevelsList = PinnedWorldModel->GetSelectedLevels();
			PinnedWorldModel->GenerateLODLevels(LevelsList, LODIndex);
		}
	}
	
	return FReply::Handled();
}

bool FTileLODEntryDetailsCustomization::IsGenerateTileEnabled() const
{
	TSharedPtr<FWorldTileCollectionModel> PinnedWorldModel = WorldModel.Pin();
	if (PinnedWorldModel.IsValid())
	{
		return PinnedWorldModel->AreAnySelectedLevelsLoaded() && (PinnedWorldModel->HasMeshProxySupport() || PinnedWorldModel->AreAnySelectedLevelsHaveLandscape());
	}
	
	return false;
}

bool FTileLODEntryDetailsCustomization::IsLODDistanceEnabled() const
{
	TSharedPtr<FWorldTileCollectionModel> PinnedWorldModel = WorldModel.Pin();
	if (PinnedWorldModel.IsValid())
	{
		return PinnedWorldModel->AreAnySelectedLevelsEditable();
	}
	
	return false;
}

#undef LOCTEXT_NAMESPACE
