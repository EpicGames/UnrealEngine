// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "Tiles/WorldTileCollectionModel.h"
#include "IDetailCustomization.h"
#include "IPropertyTypeCustomization.h"
#include "PropertyHandle.h"

class IDetailLayoutBuilder;

/////////////////////////////////////////////////////
// FWorldTileDetailsCustomization
class FWorldTileDetailsCustomization : public IDetailCustomization
{
public:
	// Makes a new instance of this detail layout class
	static TSharedRef<IDetailCustomization> MakeInstance(TSharedRef<FWorldTileCollectionModel> InWorldData);

	// IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
	// End of IDetailCustomization interface
private:
	/** Whether property should be editable (depends on level load status and level lock status) */
	bool IsPropertyEditable() const;
	
	/** Whether LOD entry property should be visible (depends on LOD entries num) */
	EVisibility GetLODPropertyVisibility(int LODIndex) const;
	
	/** Whether a particular package should be filtered out from a parent level package property editor */
	bool OnShouldFilterParentPackage(const FString& PackageName);

private:
	TSharedPtr<IPropertyHandle>			NumLODHandle;	
	TSharedPtr<IPropertyHandle>			TileEditableHandle;
	TWeakPtr<FWorldTileCollectionModel>	WorldModel;
};

/////////////////////////////////////////////////////
// FStreamingLevelDetailsCustomization
class FStreamingLevelDetailsCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance(TSharedRef<FWorldTileCollectionModel> InWorldData);

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;
	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;

private:
	bool OnShouldFilterStreamingPackage(const FString& InPackageName) const;

private:
	TWeakPtr<FWorldTileCollectionModel> WorldModel;
};

/////////////////////////////////////////////////////
// FTileLODEntryDetailsCustomization
class FTileLODEntryDetailsCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance(TSharedRef<FWorldTileCollectionModel> InWorldData);

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader( TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;
	virtual void CustomizeChildren( TSharedRef<IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils ) override;

private:
	FReply OnGenerateTile();
	bool IsGenerateTileEnabled() const;
	bool IsLODDistanceEnabled() const;

private:
	TWeakPtr<FWorldTileCollectionModel> WorldModel;
	TSharedPtr<IPropertyHandle>			LODIndexHandle;
};
