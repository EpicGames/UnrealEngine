// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tiles/WorldTileModel.h"

#include "DragAndDrop/LevelDragDropOp.h"
#include "Editor.h"
#include "Editor/Transactor.h"
#include "EditorLevelUtils.h"
#include "Engine/LevelBounds.h"
#include "Engine/LevelStreamingDynamic.h"
#include "Engine/World.h"
#include "Engine/WorldComposition.h"
#include "GameFramework/WorldSettings.h"
#include "HAL/FileManager.h"
#include "Landscape.h"
#include "LandscapeEditorModule.h"
#include "LandscapeFileFormatInterface.h"
#include "LandscapeInfo.h"
#include "LandscapeStreamingProxy.h"
#include "LevelCollectionModel.h"
#include "Modules/ModuleManager.h"
#include "Modules/ModuleManager.h"
#include "ScopedTransaction.h"
#include "Tiles/WorldTileCollectionModel.h"
#include "Tiles/WorldTileDetails.h"
#include "UObject/Package.h"

#define LOCTEXT_NAMESPACE "WorldBrowser"

FWorldTileModel::FWorldTileModel(FWorldTileCollectionModel& InWorldModel, int32 InTileIdx)
	: FLevelModel(InWorldModel) 
	, TileIdx(InTileIdx)
	, TileDetails(NULL)
	, bWasShelved(false)
{
	UWorldComposition* WorldComposition = LevelCollectionModel.GetWorld()->WorldComposition;

	// Tile display details object, don't mark the tile details transaction on creation, its lifespan is controlled through root set
	TileDetails = NewObject<UWorldTileDetails>(GetTransientPackage(), NAME_None, RF_Transient);
	TileDetails->AddToRoot();
	// Mark it transactional afterward so modification on it are undoable.
	TileDetails->SetFlags(RF_Transactional);

	// Subscribe to tile properties changes
	// Un-subscribe in dtor if new is added!
	TileDetails->PostUndoEvent.AddRaw(this, &FWorldTileModel::OnPostUndoEvent);
	TileDetails->PositionChangedEvent.AddRaw(this, &FWorldTileModel::OnPositionPropertyChanged);
	TileDetails->ParentPackageNameChangedEvent.AddRaw(this, &FWorldTileModel::OnParentPackageNamePropertyChanged);
	TileDetails->LODSettingsChangedEvent.AddRaw(this, &FWorldTileModel::OnLODSettingsPropertyChanged);
	TileDetails->ZOrderChangedEvent.AddRaw(this, &FWorldTileModel::OnZOrderPropertyChanged);
	TileDetails->HideInTileViewChangedEvent.AddRaw(this, &FWorldTileModel::OnHideInTileViewChanged);
			
	// Initialize tile details
	if (WorldComposition->GetTilesList().IsValidIndex(TileIdx))
	{
		FWorldCompositionTile& Tile = WorldComposition->GetTilesList()[TileIdx];
		
		TileDetails->PackageName = Tile.PackageName;
		TileDetails->bPersistentLevel = false;
				
		// Asset name for storing tile thumbnail inside package
		SetAssetName(Tile.PackageName);
		//FString AssetNameString = FString::Printf(TEXT("%s %s.TheWorld:PersistentLevel"), *ULevel::StaticClass()->GetName(), *Tile.PackageName.ToString());
		FString AssetNameString = FString::Printf(TEXT("%s %s"), *UPackage::StaticClass()->GetName(), *Tile.PackageName.ToString());
		AssetName =	FName(*AssetNameString);
	
		// Assign level object in case this level already loaded
		UPackage* LevelPackage = Cast<UPackage>(StaticFindObjectFast( UPackage::StaticClass(), NULL, Tile.PackageName) );
		if (LevelPackage)
		{
			// Find the world object
			UWorld* World = UWorld::FindWorldInPackage(LevelPackage);
			if (IsValid(World))
			{
				LoadedLevel = World->PersistentLevel;
				// Enable tile properties
				TileDetails->bTileEditable = true;
				if (World->PersistentLevel->bIsVisible)
				{
					LoadedLevel.Get()->LevelBoundsActorUpdated().AddRaw(this, &FWorldTileModel::OnLevelBoundsActorUpdated);
				}
			}
		}

		TileDetails->SetInfo(Tile.Info, LoadedLevel.Get());
	}
	else
	{
		TileDetails->PackageName = LevelCollectionModel.GetWorld()->GetOutermost()->GetFName();
		TileDetails->bPersistentLevel = true;
		LoadedLevel = LevelCollectionModel.GetWorld()->PersistentLevel;
	}
}

FWorldTileModel::~FWorldTileModel()
{
	if (TileDetails)
	{
		TileDetails->PostUndoEvent.RemoveAll(this);
		TileDetails->PositionChangedEvent.RemoveAll(this);
		TileDetails->ParentPackageNameChangedEvent.RemoveAll(this);
		TileDetails->LODSettingsChangedEvent.RemoveAll(this);
		TileDetails->ZOrderChangedEvent.RemoveAll(this);
		TileDetails->HideInTileViewChangedEvent.RemoveAll(this);
				
		TileDetails->RemoveFromRoot();
		TileDetails->MarkAsGarbage();
	}

	if (LoadedLevel.IsValid())
	{
		LoadedLevel.Get()->LevelBoundsActorUpdated().RemoveAll(this);
	}
}

UObject* FWorldTileModel::GetNodeObject()
{
	// this pointer is used as unique key in SNodePanel
	return TileDetails;
}

ULevel* FWorldTileModel::GetLevelObject() const
{
	return LoadedLevel.Get();
}

bool FWorldTileModel::IsRootTile() const
{
	return TileDetails->bPersistentLevel;
}

void FWorldTileModel::SetAssetName(const FName& PackageName)
{
	FString AssetNameString = FString::Printf(TEXT("%s %s"), *UPackage::StaticClass()->GetName(), *PackageName.ToString());
	AssetName = FName(*AssetNameString);
}

FName FWorldTileModel::GetAssetName() const
{
	return AssetName;
}

FName FWorldTileModel::GetLongPackageName() const
{
	return TileDetails->PackageName;
}

void FWorldTileModel::UpdateAsset(const FAssetData& AssetData)
{
	check(TileDetails != nullptr);
	const FName OldPackageName = TileDetails->PackageName;

	// Patch up any parent references which have been renamed
	for (const TSharedPtr<FLevelModel>& LevelModel : LevelCollectionModel.GetAllLevels())
	{
		TSharedPtr<FWorldTileModel> WorldTileModel = StaticCastSharedPtr<FWorldTileModel>(LevelModel);

		check(WorldTileModel->TileDetails != nullptr);
		if (WorldTileModel->TileDetails->ParentPackageName == OldPackageName)
		{
			WorldTileModel->TileDetails->ParentPackageName = AssetData.PackageName;
		}
	}

	const FName PackageName = AssetData.PackageName;
	SetAssetName(PackageName);
	TileDetails->PackageName = PackageName;
}

FVector2D FWorldTileModel::GetLevelPosition2D() const
{
	if (TileDetails->Bounds.IsValid && !TileDetails->bHideInTileView)
	{
		FVector LevelPosition = GetLevelCurrentPosition();
		FVector2D LevelPosition2D = FVector2D(LevelPosition);
		return LevelPosition2D - FVector2D(TileDetails->Bounds.GetExtent()) + GetLevelTranslationDelta();
	}

	return FVector2D(0, 0);
}

FVector2D FWorldTileModel::GetLevelSize2D() const
{
	if (TileDetails->Bounds.IsValid && !TileDetails->bHideInTileView)
	{
		FVector LevelSize = TileDetails->Bounds.GetSize();
		return FVector2D(LevelSize.X, LevelSize.Y);
	}
	
	return FVector2D(-1, -1);
}

void FWorldTileModel::OnDrop(const TSharedPtr<FLevelDragDropOp>& Op)
{
	FLevelModelList LevelModelList;

	for (auto It = Op->LevelsToDrop.CreateConstIterator(); It; ++It)
	{
		ULevel* Level = (*It).Get();
		TSharedPtr<FLevelModel> LevelModel = LevelCollectionModel.FindLevelModel(Level);
		if (LevelModel.IsValid())
		{
			LevelModelList.Add(LevelModel);
		}
	}	
	
	if (LevelModelList.Num())
	{
		const FScopedTransaction AssignParentTransaction(LOCTEXT("AssignParentTransaction", "Assign Parent Level"));
		LevelCollectionModel.AssignParent(LevelModelList, this->AsShared());
	}
}

bool FWorldTileModel::IsGoodToDrop(const TSharedPtr<FLevelDragDropOp>& Op) const
{
	return true;
}

bool FWorldTileModel::ShouldBeVisible(FBox EditableArea) const
{
	if (IsRootTile())
	{
		return true;
	}

	// Visibility does not depend on level positions when world origin rebasing is disabled
	if (!LevelCollectionModel.IsOriginRebasingEnabled())
	{
		return true;
	}

	// When this hack is activated level should be visible regardless of current world origin
	if (LevelCollectionModel.GetWorld()->WorldComposition->bTemporarilyDisableOriginTracking)
	{
		return true;
	}
	
	FBox LevelBBox = GetLevelBounds();

	// Visible if level has no valid bounds
	if (!LevelBBox.IsValid)
	{
		return true;
	}

	// Visible if level bounds inside editable area
	if (EditableArea.IsInsideXY(LevelBBox))
	{
		return true;
	}

	// Visible if level bounds intersects editable area
	if (LevelBBox.IntersectXY(EditableArea))
	{
		return true;
	}

	return false;
}

void FWorldTileModel::SetVisibleInEditor(bool bVisible)
{
	if (LevelCollectionModel.IsReadOnly())
	{
		return;
	}
		
	ULevel* Level = GetLevelObject();
	
	if (Level == NULL)
	{
		return;
	}
	
	// Don't create unnecessary transactions
	if (IsVisibleInEditor() == bVisible)
	{
		return;
	}

	// Can not show level outside of editable area
	if (bVisible && !ShouldBeVisible(LevelCollectionModel.EditableWorldArea()))
	{
		return;
	}
	
	// The level is no longer shelved
	bWasShelved = false;
	
	{
		FScopedTransaction Transaction( LOCTEXT("ToggleVisibility", "Toggle Level Visibility") );
		
		// This call hides/shows all owned actors, etc
		EditorLevelUtils::SetLevelVisibility( Level, bVisible, true );	// we need to enable layers too here so the LODs export correctly
			
		// Ensure operation is completed succesfully
		check(GetLevelObject()->bIsVisible == bVisible);

		// Now there is no way to correctly undo level visibility
		// remove ability to undo this operation
		Transaction.Cancel();
	}
}

bool FWorldTileModel::IsShelved() const
{
	return (GetLevelObject() == NULL || bWasShelved);
}

void FWorldTileModel::Shelve()
{
	if (LevelCollectionModel.IsReadOnly() || IsShelved() || IsRootTile() || !LevelCollectionModel.IsOriginRebasingEnabled())
	{
		return;
	}
	
	//
	SetVisibleInEditor(false);
	bWasShelved = true;
}

void FWorldTileModel::Unshelve()
{
	if (LevelCollectionModel.IsReadOnly() || !IsShelved())
	{
		return;
	}

	//
	SetVisibleInEditor(true);
	bWasShelved = false;
}

bool FWorldTileModel::IsLandscapeBased() const
{
	return Landscape.IsValid();
}

bool FWorldTileModel::IsTiledLandscapeBased() const
{
	return CanReimportHeightmap();
}

bool FWorldTileModel::CanReimportHeightmap() const
{
	if (IsLandscapeBased() && !GetLandscape()->ReimportHeightmapFilePath.IsEmpty())
	{
		// Check if single landscape actor resolution matches heightmap file size
		ILandscapeEditorModule& LandscapeEditorModule = FModuleManager::GetModuleChecked<ILandscapeEditorModule>("LandscapeEditor");
		const FString TargetExtension = FPaths::GetExtension(GetLandscape()->ReimportHeightmapFilePath, true);
		const ILandscapeHeightmapFileFormat* HeightmapFormat = LandscapeEditorModule.GetHeightmapFormatByExtension(*TargetExtension);

		FLandscapeFileInfo HeightmapInfo = HeightmapFormat->Validate(*GetLandscape()->ReimportHeightmapFilePath, NAME_None);
		if (HeightmapInfo.ResultCode != ELandscapeImportResult::Error)
		{
			FIntRect ComponentsRect = GetLandscape()->GetBoundingRect();

			for (FLandscapeFileResolution& PossibleResolution: HeightmapInfo.PossibleResolutions)
			{
				if ((PossibleResolution.Width == (ComponentsRect.Width() + 1)) && (PossibleResolution.Height == (ComponentsRect.Height() + 1)))
				{
					return true;
				}
			}
		}
	}

	return false;
}

bool FWorldTileModel::IsLandscapeProxy() const
{
	return IsLandscapeStreamingProxy();
}

bool FWorldTileModel::IsLandscapeStreamingProxy() const
{
	return (Landscape.IsValid() && Landscape.Get()->IsA(ALandscapeStreamingProxy::StaticClass()));
}

bool FWorldTileModel::IsInLayersList(const TArray<FWorldTileLayer>& InLayerList) const
{
	if (InLayerList.Num() > 0)
	{
		return InLayerList.Contains(TileDetails->Layer);
	}
	
	return true;
}

ALandscapeProxy* FWorldTileModel::GetLandscape() const
{
	return Landscape.Get();
}

void FWorldTileModel::AssignToLayer(const FWorldTileLayer& InLayer)
{
	if (LevelCollectionModel.IsReadOnly())
	{
		return;
	}
	
	if (!IsRootTile() && IsLoaded())
	{
		TileDetails->Layer = InLayer;
		OnLevelInfoUpdated();
	}
}

FBox FWorldTileModel::GetLevelBounds() const
{
	// Level local bounding box
	FBox Bounds = TileDetails->Bounds;

	if (Bounds.IsValid)
	{
		// Current level position in the world
		FVector LevelPosition = GetLevelCurrentPosition();
		FVector LevelExtent = Bounds.GetExtent();
		// Calculate bounding box in world space
		Bounds.Min = LevelPosition - LevelExtent;
		Bounds.Max = LevelPosition + LevelExtent;
	}
	
	return Bounds;
}

FIntVector FWorldTileModel::CalcAbsoluteLevelPosition() const
{
	TSharedPtr<FWorldTileModel> ParentModel = StaticCastSharedPtr<FWorldTileModel>(GetParent());
	if (ParentModel.IsValid())
	{
		return TileDetails->Position + ParentModel->CalcAbsoluteLevelPosition();
	}

	return IsRootTile() ? FIntVector::ZeroValue : TileDetails->Position;
}

FIntVector FWorldTileModel::GetAbsoluteLevelPosition() const
{
	return IsRootTile() ? FIntVector::ZeroValue : TileDetails->AbsolutePosition;
}
	
FIntVector FWorldTileModel::GetRelativeLevelPosition() const
{
	return IsRootTile() ? FIntVector::ZeroValue : TileDetails->Position;
}

FVector FWorldTileModel::GetLevelCurrentPosition() const
{
	if (TileDetails->Bounds.IsValid)
	{
		UWorld* CurrentWorld = (LevelCollectionModel.IsSimulating() ? LevelCollectionModel.GetSimulationWorld() : LevelCollectionModel.GetWorld());

		FVector LevelLocalPosition(TileDetails->Bounds.GetCenter());
		FIntVector LevelOffset = GetAbsoluteLevelPosition();
			
		return LevelLocalPosition + FVector(LevelOffset - CurrentWorld->OriginLocation); 
	}

	return FVector::ZeroVector;
}

void FWorldTileModel::SetLevelPosition(const FIntVector& InPosition, const FIntPoint* InLandscapeSectionOffset)
{
	// Parent absolute position
	TSharedPtr<FWorldTileModel> ParentModel = StaticCastSharedPtr<FWorldTileModel>(GetParent());
	FIntVector ParentAbsolutePostion = ParentModel.IsValid() ? ParentModel->GetAbsoluteLevelPosition() : FIntVector::ZeroValue;
	
	// Actual offset
	FIntVector Offset = InPosition - TileDetails->AbsolutePosition;
	FIntPoint LandscapeOffset = InLandscapeSectionOffset ? *InLandscapeSectionOffset : FIntPoint(Offset.X, Offset.Y);

	TileDetails->Modify();
	
	// Update absolute position
	TileDetails->AbsolutePosition = InPosition;

	// Assign new position as relative to parent
	TileDetails->Position = TileDetails->AbsolutePosition - ParentAbsolutePostion;
	
	// Flush changes to level package
	OnLevelInfoUpdated();
	
	// Move actors if necessary
	ULevel* Level = GetLevelObject();
	if (Level != nullptr && Level->bIsVisible)
	{
		// Shelve level, if during this translation level will end up out of Editable area
		if (!ShouldBeVisible(LevelCollectionModel.EditableWorldArea()))
		{
			Shelve();
		}
		
		// Move actors
		if (Offset != FIntVector::ZeroValue)
		{
			Level->ApplyWorldOffset(FVector(Offset), false);

			for (AActor* Actor : Level->Actors)
			{
				if (Actor != nullptr)
				{
					GEditor->BroadcastOnActorMoved(Actor);
				}
			}
		}
	}

	if (IsLandscapeBased())
	{
		UpdateLandscapeSectionsOffset(LandscapeOffset); // section offset is 2D 
		bool bShowWarnings = true;
		ULandscapeInfo::RecreateLandscapeInfo(LevelCollectionModel.GetWorld(), bShowWarnings);
	}
	
	// Transform child levels
	for (auto It = AllChildren.CreateIterator(); It; ++It)
	{
		TSharedPtr<FWorldTileModel> ChildModel = StaticCastSharedPtr<FWorldTileModel>(*It);
		FIntVector ChildPosition = TileDetails->AbsolutePosition + ChildModel->GetRelativeLevelPosition();
		ChildModel->SetLevelPosition(ChildPosition);
	}
}

void FWorldTileModel::UpdateLandscapeSectionsOffset(FIntPoint LevelOffset)
{
	ALandscapeProxy* LandscapeProxy = GetLandscape();
	if (LandscapeProxy)
	{
		// Calculate new section coordinates for landscape
		FVector DrawScale = LandscapeProxy->GetRootComponent()->GetRelativeScale3D();
		FIntPoint QuadsSpaceOffset;
		QuadsSpaceOffset.X = FMath::RoundToInt(LevelOffset.X / DrawScale.X);
		QuadsSpaceOffset.Y = FMath::RoundToInt(LevelOffset.Y / DrawScale.Y);
		LandscapeProxy->SetSectionBase(QuadsSpaceOffset + LandscapeProxy->GetSectionBase());
	}
}

void FWorldTileModel::Update()
{
	if (!IsRootTile())
	{
		Landscape = NULL;
		
		ULevel* Level = GetLevelObject();
		// Receive tile info from world composition
		FWorldTileInfo Info = LevelCollectionModel.GetWorld()->WorldComposition->GetTileInfo(TileDetails->PackageName);
		TileDetails->SetInfo(Info, Level);
						
		if (Level != nullptr && Level->bIsVisible)
		{
			if (Level->LevelBoundsActor.IsValid())
			{
				TileDetails->Bounds = Level->LevelBoundsActor.Get()->GetComponentsBoundingBox();
			}
								
			// True level bounds without offsets applied
			if (TileDetails->Bounds.IsValid)
			{
				FBox LevelWorldBounds = TileDetails->Bounds;
				FIntVector LevelAbolutePosition = GetAbsoluteLevelPosition();
				FIntVector LevelOffset = LevelAbolutePosition - LevelCollectionModel.GetWorld()->OriginLocation;

				TileDetails->Bounds = LevelWorldBounds.ShiftBy(-FVector(LevelOffset));
			}

			OnLevelInfoUpdated();

			// Cache landscape information
			for (int32 ActorIndex = 0; ActorIndex < Level->Actors.Num() ; ++ActorIndex)
			{
				AActor* Actor = Level->Actors[ActorIndex];
				if (Actor)
				{
					ALandscapeProxy* LandscapeActor = Cast<ALandscapeProxy>(Actor);
					if (LandscapeActor)
					{
						Landscape = LandscapeActor;
						break;
					}
				}
			}
		}
	}

	FLevelModel::Update();
}

void FWorldTileModel::LoadLevel()
{
	// Level is currently loading or has been loaded already
	if (bLoadingLevel || LoadedLevel.IsValid())
	{
		return;
	}

	// Create transient level streaming object and add to persistent level
	ULevelStreaming* LevelStreaming = GetAssociatedStreamingLevel();
	// should be clean level streaming object here
	check(LevelStreaming && LevelStreaming->GetLoadedLevel() == nullptr);
	
	bLoadingLevel = true;

	// Load level package 
	{
		FName LevelPackageName = LevelStreaming->GetWorldAssetPackageFName();
		
		ULevel::StreamedLevelsOwningWorld.Add(LevelPackageName, LevelCollectionModel.GetWorld());
		UWorld::WorldTypePreLoadMap.FindOrAdd(LevelPackageName) = LevelCollectionModel.GetWorld()->WorldType;

		UPackage* LevelPackage = LoadPackage(nullptr, *LevelPackageName.ToString(), LOAD_None);

		ULevel::StreamedLevelsOwningWorld.Remove(LevelPackageName);
		UWorld::WorldTypePreLoadMap.Remove(LevelPackageName);

		// Find world object and use its PersistentLevel pointer.
		UWorld* LevelWorld = UWorld::FindWorldInPackage(LevelPackage);
		// Check for a redirector. Follow it, if found.
		if (LevelWorld == nullptr)
		{
			LevelWorld = UWorld::FollowWorldRedirectorInPackage(LevelPackage);
		}

		if (LevelWorld && LevelWorld->PersistentLevel)
		{
			// LevelStreaming is transient object so world composition stores color in ULevel object
			LevelStreaming->LevelColor = LevelWorld->PersistentLevel->LevelColor;

#if WITH_EDITOR
			// If the level was already loaded, we need to make sure we patch the position properly
			//  because it will be first removed, then re-added to the world down below in the FlushLevelStreaming call.
			if (LevelWorld->PersistentLevel->bIsVisible && LevelWorld->PersistentLevel->OwningWorld != LevelCollectionModel.GetWorld())
			{
				if (UWorldComposition* OldWorldComposition = LevelWorld->WorldComposition)
				{
					OldWorldComposition->OnLevelRemovedFromWorld(LevelWorld->PersistentLevel);
				}
				
				OnLevelInfoUpdated(); // Force update
				
				if (UWorldComposition* NewWorldComposition = LevelCollectionModel.GetWorld()->WorldComposition)
				{
					NewWorldComposition->OnLevelAddedToWorld(LevelWorld->PersistentLevel);
				}
			}
#endif // WITH_EDITOR
		}
	}

	// Our level package should be loaded at this point, so level streaming will find it in memory
	LevelStreaming->SetShouldBeLoaded(true);
	LevelStreaming->SetShouldBeVisible(false); // Should be always false in the Editor
	LevelStreaming->SetShouldBeVisibleInEditor(false);
	LevelCollectionModel.GetWorld()->FlushLevelStreaming();

	LoadedLevel = LevelStreaming->GetLoadedLevel();
	
	bWasShelved = false;
	// Bring level to world
	if (LoadedLevel.IsValid())
	{
		// SetLevelVisibility will attempt to mark level as dirty for Undo purposes 
		// We don't want to undo sub-level loading operation, and in general loading sub-level should not make it Dirty
		FUnmodifiableObject ImmuneLevel(LoadedLevel.Get());

		// Whether this tile should be made visible at current world bounds
		const bool bShouldBeVisible = ShouldBeVisible(LevelCollectionModel.EditableWorldArea());
		EditorLevelUtils::SetLevelVisibility(LoadedLevel.Get(), bShouldBeVisible, true);
		
		// Mark tile as shelved in case it is hidden(does not fit to world bounds)
		bWasShelved = !bShouldBeVisible;
	}

	bLoadingLevel = false;
	
	// Enable tile properties
	TileDetails->bTileEditable = LoadedLevel.IsValid();

	if ((GEditor != nullptr) && (GEditor->Trans != nullptr))
	{
		GEditor->Trans->Reset(LOCTEXT("Loaded", "Discard undo history."));
	}
}

ULevelStreaming* FWorldTileModel::GetAssociatedStreamingLevel()
{
	FName PackageName = TileDetails->PackageName;
	UWorld* PersistentWorld = LevelCollectionModel.GetWorld();
				
	// Try to find existing object first
	auto Predicate = [&](ULevelStreaming* StreamingLevel) 
	{
		return (StreamingLevel && StreamingLevel->GetWorldAssetPackageFName() == PackageName && StreamingLevel->HasAnyFlags(RF_Transient));
	};

	ULevelStreaming* AssociatedStreamingLevel = nullptr;

	if (ULevelStreaming*const* FoundStreamingLevel = PersistentWorld->GetStreamingLevels().FindByPredicate(Predicate))
	{
		AssociatedStreamingLevel = *FoundStreamingLevel;
	}
	
	if (AssociatedStreamingLevel == nullptr)
	{
		// Create new streaming level
		AssociatedStreamingLevel = NewObject<ULevelStreamingDynamic>(PersistentWorld, NAME_None, RF_Transient);

		AssociatedStreamingLevel->SetWorldAssetByPackageName(PackageName);
		AssociatedStreamingLevel->LevelColor		= GetLevelColor();
		AssociatedStreamingLevel->LevelTransform	= FTransform::Identity;
		AssociatedStreamingLevel->PackageNameToLoad	= PackageName;

		PersistentWorld->AddStreamingLevel(AssociatedStreamingLevel);
	}

	return AssociatedStreamingLevel;
}

void FWorldTileModel::OnLevelAddedToWorld(ULevel* InLevel)
{
	if (!LoadedLevel.IsValid())
	{
		LoadedLevel = InLevel;
	}
		
	FLevelModel::OnLevelAddedToWorld(InLevel);

	EnsureLevelHasBoundsActor();

	// Manually call Update to make sure WorldTileModel is properly initialized (don't rely on Level Bounds changes as it will not be called if bAutoUpdateBounds is set to false).
	Update();

	LoadedLevel.Get()->LevelBoundsActorUpdated().AddRaw(this, &FWorldTileModel::OnLevelBoundsActorUpdated);
}

void FWorldTileModel::OnLevelRemovedFromWorld()
{
	FLevelModel::OnLevelRemovedFromWorld();

	ULevel* ThisLevel = LoadedLevel.Get();
	if (ThisLevel)
	{
		ThisLevel->LevelBoundsActorUpdated().RemoveAll(this);
	}
}

void FWorldTileModel::OnParentChanged()
{
	TileDetails->Modify();

	// Transform level offset to absolute
	TileDetails->Position = GetAbsoluteLevelPosition();
	// Remove link to parent	
	TileDetails->ParentPackageName = NAME_None;
	
	// Attach to new parent
	TSharedPtr<FWorldTileModel> NewParentTileModel = StaticCastSharedPtr<FWorldTileModel>(GetParent());
	if (!NewParentTileModel->IsRootTile())
	{
		// Transform level offset to relative
		TileDetails->Position-= NewParentTileModel->GetAbsoluteLevelPosition();
		// Setup link to parent 
		TileDetails->ParentPackageName = NewParentTileModel->TileDetails->PackageName;
	}
		
	OnLevelInfoUpdated();
}

bool FWorldTileModel::IsVisibleInCompositionView() const
{
	return !TileDetails->bHideInTileView && LevelCollectionModel.PassesAllFilters(*this);
}

FLinearColor FWorldTileModel::GetLevelColor() const
{
	ULevel* LevelObject = GetLevelObject();
	if (LevelObject)
	{
		return LevelObject->LevelColor;
	}
	else
	{
		return FLevelModel::GetLevelColor();
	}
}

void FWorldTileModel::SetLevelColor(FLinearColor InColor)
{
	ULevel* LevelObject = GetLevelObject();
	if (LevelObject)
	{
		ULevelStreaming* StreamingLevel = GetAssociatedStreamingLevel();
		if (StreamingLevel)
		{
			LevelObject->MarkPackageDirty();
			LevelObject->LevelColor = InColor;
			StreamingLevel->LevelColor = InColor; // this is transient object, but components fetch color from it
			LevelObject->MarkLevelComponentsRenderStateDirty();
		}
	}
}

void FWorldTileModel::OnLevelBoundsActorUpdated()
{
	Update();
}

void FWorldTileModel::EnsureLevelHasBoundsActor()
{
	ULevel* Level = GetLevelObject();
	if (Level && !Level->LevelBoundsActor.IsValid())
	{
		FActorSpawnParameters SpawnParameters;
		SpawnParameters.OverrideLevel = Level;

		LevelCollectionModel.GetWorld()->SpawnActor<ALevelBounds>(SpawnParameters);
	}
}

void FWorldTileModel::SortRecursive()
{
	AllChildren.Sort(FCompareByLongPackageName());
	FilteredChildren.Sort(FCompareByLongPackageName());
	
	for (auto It = AllChildren.CreateIterator(); It; ++It)
	{
		StaticCastSharedPtr<FWorldTileModel>(*It)->SortRecursive();
	}
}

void FWorldTileModel::OnLevelInfoUpdated()
{
	if (!IsRootTile())
	{
		LevelCollectionModel.GetWorld()->WorldComposition->OnTileInfoUpdated(TileDetails->PackageName, TileDetails->GetInfo());
		ULevel* Level = GetLevelObject();
		if (Level)
		{
			bool bMarkDirty = false;
			bMarkDirty|= !(Level->LevelSimplification[0] == TileDetails->LOD1.SimplificationDetails);
			bMarkDirty|= !(Level->LevelSimplification[1] == TileDetails->LOD2.SimplificationDetails);
			bMarkDirty|= !(Level->LevelSimplification[2] == TileDetails->LOD3.SimplificationDetails);
			bMarkDirty|= !(Level->LevelSimplification[3] == TileDetails->LOD4.SimplificationDetails);
			
			if (bMarkDirty)
			{
				Level->LevelSimplification[0] = TileDetails->LOD1.SimplificationDetails;
				Level->LevelSimplification[1] = TileDetails->LOD2.SimplificationDetails;
				Level->LevelSimplification[2] = TileDetails->LOD3.SimplificationDetails;
				Level->LevelSimplification[3] = TileDetails->LOD4.SimplificationDetails;
				Level->MarkPackageDirty();
			}
		}
	}
}

void FWorldTileModel::OnPostUndoEvent()
{
	FWorldTileInfo Info = LevelCollectionModel.GetWorld()->WorldComposition->GetTileInfo(TileDetails->PackageName);
	if (GetLevelObject())
	{
		// Level position changes
		FIntVector NewAbsolutePosition = TileDetails->AbsolutePosition;
		if (Info.AbsolutePosition != NewAbsolutePosition)
		{
			// SetLevelPosition will update AbsolutePosition to an actual value once level is moved
			TileDetails->AbsolutePosition = Info.AbsolutePosition; 
			SetLevelPosition(NewAbsolutePosition);
		}
		
		// Level attachment changes
		FName NewParentName = TileDetails->ParentPackageName;
		if (Info.ParentTilePackageName != NewParentName.ToString())
		{
			OnParentPackageNamePropertyChanged();
		}
	}
	
	OnLevelInfoUpdated();
}

void FWorldTileModel::OnPositionPropertyChanged()
{
	FWorldTileInfo Info = LevelCollectionModel.GetWorld()->WorldComposition->GetTileInfo(TileDetails->PackageName);

	if (GetLevelObject())
	{
		// Get the delta
		FIntVector Delta = TileDetails->Position - Info.Position;

		// Snap the delta
		FLevelModelList LevelsList; LevelsList.Add(this->AsShared());
		FVector2D SnappedDelta2D = LevelCollectionModel.SnapTranslationDelta(LevelsList, FVector2D(Delta.X, Delta.Y), false, 0.f);

		// Set new level position
		SetLevelPosition(Info.AbsolutePosition + FIntVector(SnappedDelta2D.X, SnappedDelta2D.Y, Delta.Z));
		return;
	}
	
	// Restore original value
	TileDetails->Position = Info.Position;
}

void FWorldTileModel::OnParentPackageNamePropertyChanged()
{	
	if (GetLevelObject())
	{
		TSharedPtr<FLevelModel> NewParent = LevelCollectionModel.FindLevelModel(TileDetails->ParentPackageName);
		// Assign to a root level if new parent is not found or we assigning to ourself 
		if (!NewParent.IsValid() || NewParent.Get() == this) 
		{
			NewParent = static_cast<FWorldTileCollectionModel&>(LevelCollectionModel).GetWorldRootModel();
		}

		FLevelModelList LevelList; LevelList.Add(this->AsShared());
		LevelCollectionModel.AssignParent(LevelList, NewParent);
		return;
	}
	
	// Restore original parent
	FWorldTileInfo Info = LevelCollectionModel.GetWorld()->WorldComposition->GetTileInfo(TileDetails->PackageName);
	TileDetails->ParentPackageName = FName(*Info.ParentTilePackageName);
}

void FWorldTileModel::OnLODSettingsPropertyChanged()
{
	OnLevelInfoUpdated();
}

void FWorldTileModel::OnZOrderPropertyChanged()
{
	OnLevelInfoUpdated();
}

void FWorldTileModel::OnHideInTileViewChanged()
{
	OnLevelInfoUpdated();
}

bool FWorldTileModel::CreateAdjacentLandscapeProxy(ALandscapeProxy* SourceLandscape, FWorldTileModel::EWorldDirections InWhere)
{
	if (!IsLoaded())	
	{
		return false;
	}

	// Determine import parameters from source landscape
	FBox SourceLandscapeBounds = SourceLandscape->GetComponentsBoundingBox(true);
	FVector SourceLandscapeScale = SourceLandscape->GetRootComponent()->GetComponentToWorld().GetScale3D();
	FIntRect SourceLandscapeRect = SourceLandscape->GetBoundingRect();
	FIntPoint SourceLandscapeSize = SourceLandscapeRect.Size();
	FIntPoint LandscapeSectionOffset = FIntPoint((SourceLandscape->GetSectionBase().X+SourceLandscapeRect.Min.X)*SourceLandscapeScale.X,
												 (SourceLandscape->GetSectionBase().Y+SourceLandscapeRect.Min.Y)*SourceLandscapeScale.Y);

	FLandscapeImportSettings ImportSettings = {};
	ImportSettings.LandscapeGuid		= SourceLandscape->GetLandscapeGuid();
	ImportSettings.LandscapeMaterial	= SourceLandscape->GetLandscapeMaterial();
	ImportSettings.ComponentSizeQuads	= SourceLandscape->ComponentSizeQuads;
	ImportSettings.QuadsPerSection		= SourceLandscape->SubsectionSizeQuads;
	ImportSettings.SectionsPerComponent = SourceLandscape->NumSubsections;
	ImportSettings.SizeX				= SourceLandscapeRect.Width() + 1;
	ImportSettings.SizeY				= SourceLandscapeRect.Height() + 1;

	// Initialize with blank heightmap data
	ImportSettings.HeightData.AddUninitialized(ImportSettings.SizeX * ImportSettings.SizeY);
	for (auto& HeightSample : ImportSettings.HeightData)
	{
		HeightSample = 32768;
	}

	// Set proxy location at landscape bounds Min point
	ImportSettings.LandscapeTransform.SetLocation(FVector(0.f, 0.f, SourceLandscape->GetActorLocation().Z));
	ImportSettings.LandscapeTransform.SetScale3D(SourceLandscapeScale);
	
	// Create new landscape object
	ALandscapeProxy* AdjacentLandscape = ImportLandscapeTile(ImportSettings);
	if (AdjacentLandscape)
	{
		// Copy source landscape properties 
		AdjacentLandscape->CopySharedProperties(SourceLandscape);
		
		// Refresh level model bounding box
		FBox AdjacentLandscapeBounds = AdjacentLandscape->GetComponentsBoundingBox(true);
		TileDetails->Bounds = AdjacentLandscapeBounds;

		// Calculate proxy offset from source landscape actor
		FVector ProxyOffset(SourceLandscapeBounds.GetCenter() - AdjacentLandscapeBounds.GetCenter());

		// Add offset by chosen direction
		switch (InWhere)
		{
		case FWorldTileModel::XNegative:
			LandscapeSectionOffset += FIntPoint(-SourceLandscapeScale.X*SourceLandscapeSize.X, 0);
			ProxyOffset += FVector(-SourceLandscapeScale.X*SourceLandscapeSize.X, 0.f, 0.f);
			break;
		case FWorldTileModel::XPositive:
			LandscapeSectionOffset += FIntPoint(+SourceLandscapeScale.X*SourceLandscapeSize.X, 0);
			ProxyOffset += FVector(+SourceLandscapeScale.X*SourceLandscapeSize.X, 0.f, 0.f);
			break;
		case FWorldTileModel::YNegative:
			LandscapeSectionOffset += FIntPoint(0, -SourceLandscapeScale.Y*SourceLandscapeSize.Y);
			ProxyOffset += FVector(0.f, -SourceLandscapeScale.Y*SourceLandscapeSize.Y, 0.f);
			break;
		case FWorldTileModel::YPositive:
			LandscapeSectionOffset += FIntPoint(0, +SourceLandscapeScale.Y*SourceLandscapeSize.Y);
			ProxyOffset += FVector(0.f, +SourceLandscapeScale.Y*SourceLandscapeSize.Y, 0.f);
			break;
		}

		// Add source level position
		FIntVector IntOffset = FIntVector(ProxyOffset) + LevelCollectionModel.GetWorld()->OriginLocation;

		// Move level with landscape proxy to desired position
		SetLevelPosition(IntOffset, &LandscapeSectionOffset);
		return true;
	}

	return false;
}

ALandscapeProxy* FWorldTileModel::ImportLandscapeTile(const FLandscapeImportSettings& Settings)
{
	if (!IsLoaded())
	{
		return nullptr;
	}
	
	check(Settings.LandscapeGuid.IsValid())
	
	ALandscapeProxy* LandscapeProxy = Cast<UWorld>(LoadedLevel->GetOuter())->SpawnActor<ALandscapeStreamingProxy>();
	LandscapeProxy->SetActorTransform(Settings.LandscapeTransform);
		
	if (Settings.LandscapeMaterial)
	{
		LandscapeProxy->LandscapeMaterial = Settings.LandscapeMaterial;
	}
	
	// Cache pointer to landscape in the level model
	Landscape = LandscapeProxy;

	TMap<FGuid, TArray<uint16>> HeightmapDataPerLayers;
	TMap<FGuid, TArray<FLandscapeImportLayerInfo>> MaterialLayerDataPerLayer;

	HeightmapDataPerLayers.Add(FGuid(), Settings.HeightData);
	MaterialLayerDataPerLayer.Add(FGuid(), Settings.ImportLayers);	

	// Create landscape components
	LandscapeProxy->Import(	Settings.LandscapeGuid, 0, 0, Settings.SizeX - 1, Settings.SizeY - 1, Settings.SectionsPerComponent, Settings.QuadsPerSection, HeightmapDataPerLayers, *Settings.HeightmapFilename,	
							MaterialLayerDataPerLayer,	Settings.ImportLayerType, TArrayView<const FLandscapeLayer>());

	for (const FLandscapeImportLayerInfo& ImportLayerInfo : Settings.ImportLayers)
	{
		if (ImportLayerInfo.LayerInfo != nullptr)
		{
			LandscapeProxy->AddTargetLayer(ImportLayerInfo.LayerInfo->GetLayerName(), FLandscapeTargetLayerSettings(ImportLayerInfo.LayerInfo, ImportLayerInfo.SourceFilePath));
		}
	}

	return LandscapeProxy;
}

#undef LOCTEXT_NAMESPACE
