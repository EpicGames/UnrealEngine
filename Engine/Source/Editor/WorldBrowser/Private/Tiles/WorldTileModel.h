// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "Misc/WorldCompositionUtility.h"
#include "LevelModel.h"
#include "LandscapeProxy.h"

#define UE_API WORLDBROWSER_API

struct FAssetData;
class FLevelDragDropOp;
class FWorldTileCollectionModel;
class FWorldTileModel;
class ULevelStreaming;
class UMaterialInterface;
class UWorldTileDetails;

//
typedef TArray<TSharedPtr<class FWorldTileModel>> FWorldTileModelList;


/**
 * The non-UI presentation logic for a single Level (Tile) in world composition
 */
class  FWorldTileModel
	: public FLevelModel	
{
public:
	struct FCompareByLongPackageName
	{
		FORCEINLINE bool operator()(const TSharedPtr<FLevelModel>& A, 
									const TSharedPtr<FLevelModel>& B) const
		{
			return A->GetLongPackageName().LexicalLess(B->GetLongPackageName());
		}
	};

	enum EWorldDirections
	{
		XNegative,
		YNegative,
		XPositive,
		YPositive,
		Any
	};

	/**
	 * 
	 */
	struct FLandscapeImportSettings
	{
		// Depending on landscape guid import code will spawn Landscape actor or LandscapeProxy actor
		FGuid								LandscapeGuid;
		FTransform							LandscapeTransform;
		UMaterialInterface*					LandscapeMaterial;
		int32								ComponentSizeQuads;
		int32								SectionsPerComponent;
		int32								QuadsPerSection;
		int32								SizeX;
		int32								SizeY;
		TArray<uint16>						HeightData;
		TArray<FLandscapeImportLayerInfo>	ImportLayers;
		FString								HeightmapFilename;
		ELandscapeImportAlphamapType		ImportLayerType;
	};
	
	/**
	 *	FWorldTileModel Constructor
	 *
	 *	@param	InWorldData		WorldBrowser world data
	 *	@param	TileIdx			Tile index in world composition tiles list
	 */
	UE_API FWorldTileModel(FWorldTileCollectionModel& InWorldData, int32 TileIdx);
	UE_API ~FWorldTileModel();

public:
	// FLevelModel interface
	UE_API virtual UObject* GetNodeObject() override;
	UE_API virtual ULevel* GetLevelObject() const override;
	UE_API virtual FName GetAssetName() const override;
	UE_API virtual FName GetLongPackageName() const override;
	UE_API virtual void Update() override;
	UE_API virtual void UpdateAsset(const FAssetData& AssetData) override;
	UE_API virtual void LoadLevel() override;
	UE_API virtual void SetVisibleInEditor(bool bVisible) override;
	UE_API virtual FVector2D GetLevelPosition2D() const override;
	UE_API virtual FVector2D GetLevelSize2D() const override;
	UE_API virtual void OnDrop(const TSharedPtr<FLevelDragDropOp>& Op) override;
	UE_API virtual bool IsGoodToDrop(const TSharedPtr<FLevelDragDropOp>& Op) const override;
	UE_API virtual void OnLevelAddedToWorld(ULevel* InLevel) override;
	UE_API virtual void OnLevelRemovedFromWorld() override;
	UE_API virtual void OnParentChanged() override;
	UE_API virtual bool IsVisibleInCompositionView() const override;
	UE_API virtual FLinearColor GetLevelColor() const override;
	UE_API virtual void SetLevelColor(FLinearColor InColor) override;
	// FLevelModel interface end
	
	/** Adds new streaming level*/
	UE_API void AddStreamingLevel(UClass* InStreamingClass, const FName& InPackageName);

	/** Assigns level to provided layer*/
	UE_API void AssignToLayer(const FWorldTileLayer& InLayer);

	/**	@return Whether tile is root of hierarchy */
	UE_API bool IsRootTile() const;

	/** Whether level should be visible in given area*/
	UE_API bool ShouldBeVisible(FBox EditableArea) const;

	/**	@return Whether level is shelved */
	UE_API bool IsShelved() const;

	/** Hide a level from the editor */
	UE_API void Shelve();
	
	/** Show a level in the editor */
	UE_API void Unshelve();

	/** Whether this level landscape based or not */
	UE_API bool IsLandscapeBased() const;

	UE_DEPRECATED(4.26, "No longer used; use CanReimportHeightmap instead.")
	UE_API bool IsTiledLandscapeBased() const;

	/** Whether this level has a heightmap that can be reimported or not */
	UE_API bool CanReimportHeightmap() const;

	UE_DEPRECATED(4.26, "No longer used; use IsLandscapeStreamingProxy instead.")
	UE_API bool IsLandscapeProxy() const;

	/** Whether this level has ALandscapeStreamingProxy or not */
	UE_API bool IsLandscapeStreamingProxy() const;

	/** @return The landscape actor in case this level is landscape based */
	UE_API ALandscapeProxy* GetLandscape() const;
	
	/** Whether this level in provided layers list */
	UE_API bool IsInLayersList(const TArray<FWorldTileLayer>& InLayerList) const;
	
	/** @return Level position in shifted space */
	UE_API FVector GetLevelCurrentPosition() const;

	/** @return Level relative position */
	UE_API FIntVector GetRelativeLevelPosition() const;
	
	/** @return Level absolute position in non shifted space */
	UE_API FIntVector GetAbsoluteLevelPosition() const;
	
	/** @return Calculates Level absolute position in non shifted space based on relative position */
	UE_API FIntVector CalcAbsoluteLevelPosition() const;
		
	/** @return ULevel bounding box in shifted space*/
	UE_API FBox GetLevelBounds() const override;
	
	/** Translate level center to new position */
	UE_API void SetLevelPosition(const FIntVector& InPosition, const FIntPoint* InLandscapeSectionOffset = nullptr);

	/** Recursively sort all children by name */
	UE_API void SortRecursive();

	/** 
	 *	@return associated streaming level object for this tile
	 *	Creates a new object in case it does not exists in a persistent world
	 */
	UE_API ULevelStreaming* GetAssociatedStreamingLevel();

	UE_DEPRECATED(4.20, "Call GetAssociatedStreamingLevel instead")
	ULevelStreaming* GetAssosiatedStreamingLevel() { return GetAssociatedStreamingLevel(); }

	/**  */
	UE_API bool CreateAdjacentLandscapeProxy(ALandscapeProxy* SourceLandscape, FWorldTileModel::EWorldDirections InWhere);

	/**  */
	UE_API ALandscapeProxy* ImportLandscapeTile(const FLandscapeImportSettings& Settings);

private:
	/** Flush world info to package and level objects */
	UE_API void OnLevelInfoUpdated();

	/** Fixup invalid streaming objects inside level */
	UE_API void FixupStreamingObjects();

	/** When level with landscape is moved we need to update internal landscape coordinates to match landscape component grid  */
	UE_API void UpdateLandscapeSectionsOffset(FIntPoint LevelOffset);
	
	/** Handler for LevelBoundsActorUpdated event */
	UE_API void OnLevelBoundsActorUpdated();

	/** Spawns AlevelBounds actor in the level in case it doesn't has one */
	UE_API void EnsureLevelHasBoundsActor();

	/** Handler for PostEditUndo  */
	UE_API void OnPostUndoEvent();
	
	/** Handler for PositionChanged event from Tile details object  */
	UE_API void OnPositionPropertyChanged();
	
	/** Handler for ParentPackageName event from Tile details object  */
	UE_API void OnParentPackageNamePropertyChanged();

	/** Handler for LOD settings changes event from Tile details object  */
	UE_API void OnLODSettingsPropertyChanged();
	
	/** Handler for ZOrder changes event from Tile details object  */
	UE_API void OnZOrderPropertyChanged();

	/** Handler for bHideInTileView changes event from Tile details object  */
	UE_API void OnHideInTileViewChanged();

	/** Set the asset name based on the passed in package name */
	UE_API void SetAssetName(const FName& PackageName);

public:
	/** This tile index in world composition tile list */
	int32									TileIdx;
	
	/** Package name this item represents */
	FName									AssetName;
	
	/** UObject which holds tile properties to be able to edit them via details panel*/
	UWorldTileDetails*						TileDetails;
	
	/** The Level this object represents */
	TWeakObjectPtr<ULevel>					LoadedLevel;

	// Whether this level was shelved: hidden by World Browser decision
	bool									bWasShelved;

private:
	/** Whether level has landscape components in it */
	TWeakObjectPtr<ALandscapeProxy>			Landscape;
};

#undef UE_API
