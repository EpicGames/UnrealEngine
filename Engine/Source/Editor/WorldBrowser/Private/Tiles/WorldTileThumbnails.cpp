// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tiles/WorldTileThumbnails.h"
#include "Misc/ObjectThumbnail.h"
#include "Brushes/SlateDynamicImageBrush.h"
#include "UObject/Package.h"
#include "Brushes/SlateColorBrush.h"
#include "RHI.h"
#include "RenderingThread.h"
#include "Engine/Texture2DDynamic.h"
#include "LevelCollectionModel.h"
#include "ObjectTools.h"
#include "Slate/SlateTextures.h"
#include "Tiles/WorldTileModel.h"

#include "Misc/ObjectThumbnail.inl"

static const double TileThumbnailUpdateCooldown = 0.005f;
static const FSlateColorBrush ThumbnailDefaultBrush(FLinearColor::White);

//----------------------------------------------------------------
//
//
//----------------------------------------------------------------
FTileThumbnail::FTileThumbnail(FSlateTextureRenderTarget2DResource* InThumbnailRenderTarget, const FWorldTileModel& InTileModel, const FIntPoint& InSlotAllocation)
	: TileModel(InTileModel)
	, ThumbnailRenderTarget(InThumbnailRenderTarget)
	, SlotAllocation(InSlotAllocation)
{
}

FIntPoint FTileThumbnail::GetThumbnailSlotAllocation() const
{
	return SlotAllocation;
}

FSlateTextureDataPtr FTileThumbnail::UpdateThumbnail()
{
	// No need images for persistent and always loaded levels
	if (TileModel.IsPersistent())
	{
		return ToSlateTextureData(nullptr);
	}
	
	// Load image from a package header
	if (!TileModel.IsVisibleInEditor() || TileModel.IsSimulating())
	{
		const FName LevelAssetName = TileModel.GetAssetName();
		TSet<FName> ObjectFullNames;
		ObjectFullNames.Add(LevelAssetName);
		FThumbnailMap ThumbnailMap;

		if (ThumbnailTools::ConditionallyLoadThumbnailsFromPackage(TileModel.GetPackageFileName(), ObjectFullNames, ThumbnailMap))
		{
			const FObjectThumbnail* ObjectThumbnail = ThumbnailMap.Find(LevelAssetName);
			return ToSlateTextureData(ObjectThumbnail);
		}
	}
	// Render image from a visible level
	else
	{
		ULevel* TargetLevel = TileModel.GetLevelObject();
		if (TargetLevel)
		{
			FIntPoint RTSize = ThumbnailRenderTarget->GetSizeXY();
			
			// Set persistent world package as transient to avoid package dirtying during thumbnail rendering
			FUnmodifiableObject ImmuneWorld(TargetLevel->OwningWorld);
			
			FObjectThumbnail NewThumbnail;
			// Generate the thumbnail
			ThumbnailTools::RenderThumbnail(
				TargetLevel,
				RTSize.X,
				RTSize.Y,
				ThumbnailTools::EThumbnailTextureFlushMode::NeverFlush,
				ThumbnailRenderTarget,
				&NewThumbnail
				);

			UPackage* MyOutermostPackage = TargetLevel->GetOutermost();
			ThumbnailTools::CacheThumbnail(TileModel.GetAssetName().ToString(), &NewThumbnail, MyOutermostPackage);
			return ToSlateTextureData(&NewThumbnail);
		}
	}

	return ToSlateTextureData(nullptr);
}

FSlateTextureDataPtr FTileThumbnail::ToSlateTextureData(const FObjectThumbnail* ObjectThumbnail) const
{
	FSlateTextureDataPtr Result;
	
	if (ObjectThumbnail)
	{
		FIntPoint ImageSize(ObjectThumbnail->GetImageWidth(), ObjectThumbnail->GetImageHeight());
		FIntPoint TargetSize = ThumbnailRenderTarget->GetSizeXY();
		if (TargetSize == ImageSize)
		{
			FImageView Image = ObjectThumbnail->GetImage();
			if ( Image.GetNumPixels() > 0 )
			{
				Result = MakeShareable(new FSlateTextureData(Image));
			}
		}
	}

	return Result;
}

//----------------------------------------------------------------
//
//
//----------------------------------------------------------------

FTileAtlasPage::FTileAtlasPage()
{
	AtlasTexture = UTexture2DDynamic::Create(TileThumbnailAtlasSize, TileThumbnailAtlasSize, FTexture2DDynamicCreateInfo(PF_B8G8R8A8, false));
	AtlasTexture->AddToRoot();

	static int32 NextPageUniqueID = 1;
	FName AtlasPageName = *FString::Printf(TEXT("WorldCompositionAtlasPage_%d"), NextPageUniqueID++);
	
	for (int32 i = 0; i < UE_ARRAY_COUNT(AtlasSlots); ++i)
	{
		FTileAtlasSlot& Slot = AtlasSlots[i];
		
		Slot.bOccupied = false;
		Slot.SlotBrush = new FSlateDynamicImageBrush(
			AtlasTexture, 
			FVector2D(TileThumbnailAtlasSize, TileThumbnailAtlasSize),
			AtlasPageName
			);

		int32 SlotOffsetX = i % TileThumbnailAtlasDim;
		int32 SlotOffsetY = i / TileThumbnailAtlasDim;
		FVector2D StartUV = FVector2D(SlotOffsetX, SlotOffsetY)/TileThumbnailAtlasDim;
		FVector2D SizeUV = FVector2D(1.0f, 1.0f)/TileThumbnailAtlasDim;

		Slot.SlotBrush->SetUVRegion(FBox2D(StartUV, StartUV+SizeUV));
	}
}

FTileAtlasPage::~FTileAtlasPage()
{
	for (int32 i = 0; i < UE_ARRAY_COUNT(AtlasSlots); ++i)
	{
		FTileAtlasSlot& Slot = AtlasSlots[i];
		delete Slot.SlotBrush;
		Slot.SlotBrush = nullptr;
	}
	
	AtlasTexture->RemoveFromRoot();
	AtlasTexture->MarkAsGarbage();
	AtlasTexture = nullptr;
}

void FTileAtlasPage::SetOccupied(int32 SlotIdx, bool bOccupied)
{
	AtlasSlots[SlotIdx].bOccupied = bOccupied;
}

bool FTileAtlasPage::HasOccupiedSlots() const
{
	for (int32 i = 0; i < UE_ARRAY_COUNT(AtlasSlots); ++i)
	{
		if (AtlasSlots[i].bOccupied)
		{
			return true;
		}
	}

	return false;
}

int32 FTileAtlasPage::GetFreeSlotIndex() const
{
	int32 Result = INDEX_NONE;
	for (int32 i = 0; i < UE_ARRAY_COUNT(AtlasSlots); ++i)
	{
		if (!AtlasSlots[i].bOccupied)
		{
			Result = i;
		}
	}
	return Result;
}

const FSlateBrush* FTileAtlasPage::GetSlotBrush(int32 SlotIdx) const
{
	return AtlasSlots[SlotIdx].SlotBrush;
}

void FTileAtlasPage::UpdateSlotImageData(int32 SlotIdx, FSlateTextureDataPtr ImageData)
{
	if (AtlasTexture && AtlasTexture->GetResource())
	{
		int32 SlotX = (SlotIdx % TileThumbnailAtlasDim)*TileThumbnailSize;
		int32 SlotY = (SlotIdx / TileThumbnailAtlasDim)*TileThumbnailSize;
		
		const FUpdateTextureRegion2D UpdateRegion(
			SlotX, SlotY,		// Dest X, Y
			0, 0,				// Source X, Y
			TileThumbnailSize,	// Width
			TileThumbnailSize);	// Height
	
		struct FSlotUpdateContext
		{
			FTextureRHIRef			TextureRHI;
			FSlateTextureDataPtr	ImageData;
			uint32					SourcePitch;
			FUpdateTextureRegion2D	Region;
		} 
		Context = 
		{
			AtlasTexture->GetResource()->TextureRHI,
			ImageData,
			TileThumbnailSize*4,
			UpdateRegion
		};
			
		ENQUEUE_RENDER_COMMAND(UpdateSlotImageData)(
			[Context](FRHICommandList& RHICmdList)
			{
				FRHITexture* RHITexture2D = (FRHITexture*)Context.TextureRHI.GetReference();
				RHICmdList.UpdateTexture2D(RHITexture2D, 0, Context.Region, Context.SourcePitch, Context.ImageData->GetRawBytesPtr());
			});
	}
}

//----------------------------------------------------------------
//
//
//----------------------------------------------------------------
FTileThumbnailCollection::FTileThumbnailCollection()
	: LastThumbnailUpdateTime(0.0)
{
	SharedThumbnailRT = new FSlateTextureRenderTarget2DResource(
						FLinearColor::Black, 
						TileThumbnailSize, 
						TileThumbnailSize, 
						PF_B8G8R8A8, SF_Bilinear, TA_Wrap, TA_Wrap, 0.0f
					);

	BeginInitResource(SharedThumbnailRT);
}

FTileThumbnailCollection::~FTileThumbnailCollection()
{
	for (int32 i = 0; i < AtlasPages.Num(); ++i)
	{
		delete AtlasPages[i];
		AtlasPages[i] = nullptr;
	}

	TileThumbnailsMap.Reset();
	
	BeginReleaseResource(SharedThumbnailRT);
	FlushRenderingCommands();
	delete SharedThumbnailRT;
}

void FTileThumbnailCollection::RegisterTile(const FWorldTileModel& InTileModel)
{
	FName TileName = InTileModel.GetLongPackageName();
	FIntPoint SlotAllocation = AllocateSlot();

	TileThumbnailsMap.Add(TileName, FTileThumbnail(SharedThumbnailRT, InTileModel, SlotAllocation));
}

void FTileThumbnailCollection::UnregisterTile(const FWorldTileModel& InTileModel)
{
	FName TileName = InTileModel.GetLongPackageName();
	
	const FTileThumbnail* TileThumbnail = TileThumbnailsMap.Find(TileName);
	if (TileThumbnail)
	{
		ReleaseSlot(TileThumbnail->GetThumbnailSlotAllocation());
		TileThumbnailsMap.Remove(TileName);
	}
}

const FSlateBrush* FTileThumbnailCollection::UpdateTileThumbnail(const FWorldTileModel& InTileModel)
{
	FName TileName = InTileModel.GetLongPackageName();
	FTileThumbnail* TileThumbnail = TileThumbnailsMap.Find(TileName);
	
	if (TileThumbnail)
	{
		auto ImageData = TileThumbnail->UpdateThumbnail();
		if (ImageData.IsValid())
		{
			LastThumbnailUpdateTime = FPlatformTime::Seconds();
				
			FIntPoint SlotAllocation = TileThumbnail->GetThumbnailSlotAllocation();
			AtlasPages[SlotAllocation.X]->UpdateSlotImageData(SlotAllocation.Y, ImageData);
			return AtlasPages[SlotAllocation.X]->GetSlotBrush(SlotAllocation.Y);
		}
	}

	return &ThumbnailDefaultBrush;
}

const FSlateBrush* FTileThumbnailCollection::GetTileBrush(const FWorldTileModel& InTileModel) const
{
	FName TileName = InTileModel.GetLongPackageName();
	const FTileThumbnail* TileThumbnail = TileThumbnailsMap.Find(TileName);
	
	if (TileThumbnail)
	{
		FIntPoint SlotAllocation = TileThumbnail->GetThumbnailSlotAllocation();
		return AtlasPages[SlotAllocation.X]->GetSlotBrush(SlotAllocation.Y);
	}

	return &ThumbnailDefaultBrush;
}

bool FTileThumbnailCollection::IsOnCooldown() const
{
	CA_SUPPRESS(6326);
	if (TileThumbnailUpdateCooldown > 0.0)
	{
		const double CurrentTime = FPlatformTime::Seconds();
		return ((CurrentTime - LastThumbnailUpdateTime) < TileThumbnailUpdateCooldown);
	}
	
	return false;
}

FIntPoint FTileThumbnailCollection::AllocateSlot()
{
	int32 PageIndex = INDEX_NONE;
	int32 SlotIndex = INDEX_NONE;

	for (int32 i = 0; i < AtlasPages.Num(); ++i)
	{
		SlotIndex = AtlasPages[i]->GetFreeSlotIndex();
		if (SlotIndex != INDEX_NONE)
		{
			PageIndex = i;
			break;
		}
	}

	// Add new page
	if (SlotIndex == INDEX_NONE)
	{
		PageIndex = AtlasPages.Add(new FTileAtlasPage());
		SlotIndex = AtlasPages[PageIndex]->GetFreeSlotIndex();
	}
	
	check(PageIndex!= INDEX_NONE && SlotIndex != INDEX_NONE);
	AtlasPages[PageIndex]->SetOccupied(SlotIndex, true);

	return FIntPoint(PageIndex, SlotIndex);
}

void FTileThumbnailCollection::ReleaseSlot(const FIntPoint& InSlotAllocation)
{
	AtlasPages[InSlotAllocation.X]->SetOccupied(InSlotAllocation.Y, false);
}
