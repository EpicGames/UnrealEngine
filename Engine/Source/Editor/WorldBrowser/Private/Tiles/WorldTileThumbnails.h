// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Textures/SlateTextureData.h"

class FObjectThumbnail;
class FSlateTextureRenderTarget2DResource;
class FWorldTileModel;
class UTexture2DDynamic;
struct FSlateBrush;
struct FSlateDynamicImageBrush;

//----------------------------------------------------------------
//
//
//----------------------------------------------------------------
const int32 TileThumbnailSize = 256;
const int32 TileThumbnailAtlasSize = 1024;
const int32 TileThumbnailAtlasDim = TileThumbnailAtlasSize/TileThumbnailSize;

//----------------------------------------------------------------
//
//
//----------------------------------------------------------------
class FTileThumbnail
{
public:
	FTileThumbnail(FSlateTextureRenderTarget2DResource* InThumbnailRenderTarget, const FWorldTileModel& InTileModel, const FIntPoint& InSlotAllocation);

	/** Redraw thumbnail */
	FSlateTextureDataPtr UpdateThumbnail();
	FIntPoint GetThumbnailSlotAllocation() const;

private:
	FSlateTextureDataPtr ToSlateTextureData(const FObjectThumbnail* ObjectThumbnail) const;

private:
	const FWorldTileModel&					TileModel;
	/** Shared render target for slate */
	FSlateTextureRenderTarget2DResource*	ThumbnailRenderTarget;
	FIntPoint								SlotAllocation;
};

//----------------------------------------------------------------
//
//
//----------------------------------------------------------------
class FTileAtlasPage
{
public:
	FTileAtlasPage();
	~FTileAtlasPage();
	
	void SetOccupied(int32 SlotIdx, bool bOccupied);
	bool HasOccupiedSlots() const;
	int32 GetFreeSlotIndex() const;
	const FSlateBrush* GetSlotBrush(int32 SlotIdx) const;
	void UpdateSlotImageData(int32 SlotIdx, FSlateTextureDataPtr ImageData);
		
private:
	struct FTileAtlasSlot
	{
		FSlateDynamicImageBrush*	SlotBrush;
		bool						bOccupied;
	};
	
	FTileAtlasSlot		AtlasSlots[TileThumbnailAtlasDim*TileThumbnailAtlasDim];
	UTexture2DDynamic*	AtlasTexture;
};

//----------------------------------------------------------------
//
//
//----------------------------------------------------------------
class FTileThumbnailCollection
{
public:
	FTileThumbnailCollection();
	~FTileThumbnailCollection();

	void RegisterTile(const FWorldTileModel& InTileModel);
	void UnregisterTile(const FWorldTileModel& InTileModel);
	const FSlateBrush* UpdateTileThumbnail(const FWorldTileModel& InTileModel);
	const FSlateBrush* GetTileBrush(const FWorldTileModel& InTileModel) const;

	bool IsOnCooldown() const;

private:
	FIntPoint AllocateSlot();
	void ReleaseSlot(const FIntPoint& InSlotAllocation);

private:
	FSlateTextureRenderTarget2DResource*	SharedThumbnailRT;
	TMap<FName, FTileThumbnail>				TileThumbnailsMap;
	double									LastThumbnailUpdateTime;

	TArray<FTileAtlasPage*>					AtlasPages;
};
