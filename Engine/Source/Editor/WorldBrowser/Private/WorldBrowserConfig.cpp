// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldBrowserConfig.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WorldBrowserConfig)

TObjectPtr<UWorldBrowserConfig> UWorldBrowserConfig::Instance = nullptr;

void UWorldBrowserConfig::Initialize()
{
	if(!Instance)
	{
		Instance = NewObject<UWorldBrowserConfig>(); 
		Instance->AddToRoot();
		Instance->LoadEditorConfig();
	}
}
