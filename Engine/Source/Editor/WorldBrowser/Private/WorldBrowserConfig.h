// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "EditorConfigBase.h"
#include "WorldHierarchyColumns.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "WorldBrowserConfig.generated.h"

USTRUCT()
struct FWorldBrowserColumnConfig
{
	GENERATED_BODY()

	UPROPERTY()
	TMap<FName, bool> ColumnVisibilities
	{
		// By default, hide this column; it is only used in certain workflows, like Virtual Production. The user can manually show this column.
		{ UE::WorldHierarchy::HierarchyColumns::ColumnID_GameVisibility, false }
	};
};

UCLASS(EditorConfig="WorldBrowser")
class UWorldBrowserConfig : public UEditorConfigBase
{
	GENERATED_BODY()
public:
	
	UPROPERTY(meta=(EditorConfig))
	FWorldBrowserColumnConfig ColumnConfig;

	static void Initialize();
	static UWorldBrowserConfig* Get() { return Instance; }

private:

	static TObjectPtr<UWorldBrowserConfig> Instance;
};
