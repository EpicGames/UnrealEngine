// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldBrowserDragDrop.h"
#include "Styling/AppStyle.h"
#include "WorldTreeItemTypes.h"
#include "LevelCollectionModel.h"

#define LOCTEXT_NAMESPACE "WorldBrowser"

namespace WorldHierarchy
{
	TSharedPtr<FDragDropOperation> CreateDragDropOperation(const TArray<FWorldTreeItemPtr>& InTreeItems)
	{
		TSharedPtr<FDragDropOperation> Operation;

		if (InTreeItems.Num() > 0)
		{
			// Assume that the world model for the first selected item is the same world model for all items
			TSharedPtr<FLevelCollectionModel> WorldModel = InTreeItems[0]->WorldModel.Pin();
			
			if (WorldModel.IsValid())
			{
				FDragDropPayload Payload;
				Payload.DraggedTreeItems = InTreeItems;

				// Populate the list of models that are affected by this operation
				FLevelModelList AffectedModels;

				for (FWorldTreeItemPtr Item : InTreeItems)
				{
					// Folder items should return all child models, but anything else should only return the model for that item
					if (Item->GetAsFolderTreeItem() != nullptr)
					{
						AffectedModels.Append(Item->GetLevelModels());
					}
					else
					{
						AffectedModels.Append(Item->GetModel());
					}
				}

				TSharedPtr<FWorldBrowserDragDropOp> OutlinerOp = WorldModel->CreateDragDropOp(AffectedModels);
				if (OutlinerOp.IsValid())
				{
					OutlinerOp->Init(&Payload);
					if (AffectedModels.Num() == 0)
					{
						OutlinerOp->Construct();
					}
					Operation = OutlinerOp;
				}
			}
		}

		return Operation;
	}


	FWorldBrowserDragDropOp::FWorldBrowserDragDropOp()
	{
	}

	void FWorldBrowserDragDropOp::Init(const FDragDropPayload* Payload)
	{
		FLevelDragDropOp::Init();

		if (Payload != nullptr)
		{
			DraggedItems = Payload->DraggedTreeItems;
			CurrentIconBrush = DraggedItems.Num() > 0 ? DraggedItems[0]->GetHierarchyItemBrush() : nullptr;

			if (DraggedItems.Num() == 1)
			{
				CurrentHoverText = FText::FromString(DraggedItems[0]->GetDisplayString());
			}
			else
			{
				CurrentHoverText = FText::Format(LOCTEXT("WorldHierarchyDragDrop_Default", "{0} Items"), FText::AsNumber(DraggedItems.Num()));
			}

			SetupDefaults();
		}
	}

	TSharedPtr<SWidget> FWorldBrowserDragDropOp::GetDefaultDecorator() const
	{
		TSharedRef<SVerticalBox> Decorator = SNew(SVerticalBox);

		Decorator->AddSlot()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Graph.ConnectorFeedback.Border"))
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.0f, 0.0f, 3.0f, 0.0f)
				[
					SNew(SImage)
					.Image(this, &FDecoratedDragDropOp::GetIcon)
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(this, &FDecoratedDragDropOp::GetHoverText)
				]
			]
		];

		return Decorator;
	}

}	// namespace WorldHierarchy

#undef LOCTEXT_NAMESPACE