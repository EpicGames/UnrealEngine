// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/DragAndDrop.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "DragAndDrop/LevelDragDropOp.h"

namespace WorldHierarchy
{
	struct IWorldTreeItem;
	typedef TSharedPtr<IWorldTreeItem> FWorldTreeItemPtr;
	typedef TSharedRef<IWorldTreeItem> FWorldTreeItemRef;

	struct FValidationInfo
	{
		FValidationInfo()
			: bValid(true)
		{
		}

		FText ValidationText;
		bool bValid;
	};


	struct FDragDropPayload
	{
		/** World tree items */
		TArray<WorldHierarchy::FWorldTreeItemPtr> DraggedTreeItems;
	};


	/** Construct a new drag and drop operation for the world hierarchy */
	TSharedPtr<FDragDropOperation> CreateDragDropOperation(const TArray<WorldHierarchy::FWorldTreeItemPtr>& InTreeItems);

	/** 
	 * Used to drag folders and level items within the world hierarchy widget
	 */
	struct FWorldBrowserDragDropOp : public FLevelDragDropOp
	{
		DRAG_DROP_OPERATOR_TYPE(FWorldBrowserDragDropOp, FLevelDragDropOp);

		FWorldBrowserDragDropOp();

		using FLevelDragDropOp::Construct;

		TArray<FWorldTreeItemPtr> GetDraggedItems() const { return DraggedItems; }

		/** Initializes the operation with the specified payload */
		void Init(const FDragDropPayload* Payload);

		virtual TSharedPtr<SWidget> GetDefaultDecorator() const override;

		/** Creates a drag and drop operation for the specified levels */
		static TSharedRef<FWorldBrowserDragDropOp> New(const TArray<TWeakObjectPtr<ULevel>>& Levels)
		{
			TSharedRef<FWorldBrowserDragDropOp> Op = MakeShareable(new FWorldBrowserDragDropOp);
			Op->LevelsToDrop.Append(Levels);
			Op->Init(nullptr);
			Op->Construct();
			return Op;
		}

		/** Creates a drag and drop operation for the specified streaming levels */
		static TSharedRef<FWorldBrowserDragDropOp> New(const TArray<TWeakObjectPtr<ULevelStreaming>>& StreamingLevels)
		{
			TSharedRef<FWorldBrowserDragDropOp> Op = MakeShareable(new FWorldBrowserDragDropOp);
			Op->StreamingLevelsToDrop.Append(StreamingLevels);
			Op->Init(nullptr);
			Op->Construct();
			return Op;
		}

	private:

		TArray<FWorldTreeItemPtr> DraggedItems;
	};

}	// namespace WorldHierarchy