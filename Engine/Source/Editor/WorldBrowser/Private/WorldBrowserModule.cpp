// Copyright Epic Games, Inc. All Rights Reserved.
#include "WorldBrowserModule.h"
#include "Widgets/SWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Modules/ModuleManager.h"
#include "EditorModeRegistry.h"
#include "EditorModes.h"
#include "LevelCollectionCommands.h"
#include "LevelFolders.h"

#include "Engine/WorldComposition.h"
#include "StreamingLevels/StreamingLevelEdMode.h"
#include "Tiles/WorldTileCollectionModel.h"
#include "StreamingLevels/StreamingLevelCollectionModel.h"
#include "SWorldHierarchy.h"
#include "SWorldDetails.h"
#include "Tiles/SWorldComposition.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "LevelEditor.h"
#include "EditorLevelUtils.h"
#include "ThumbnailRendering/ThumbnailManager.h"

IMPLEMENT_MODULE( FWorldBrowserModule, WorldBrowser );

#define LOCTEXT_NAMESPACE "WorldBrowser"

/**
 * Fills the level menu with extra options
 */
TSharedRef<FExtender> FWorldBrowserModule::BindLevelMenu(const TSharedRef<FUICommandList> CommandList)
{
	TSharedRef<FExtender> Extender(new FExtender());
	Extender->AddMenuExtension("LevelListing", EExtensionHook::Before, CommandList, FMenuExtensionDelegate::CreateRaw(this, &FWorldBrowserModule::BuildLevelMenu));
	return Extender;
}


void FWorldBrowserModule::BuildLevelMenu(FMenuBuilder& MenuBuilder)
{
	UWorld* EditorWorld = GEditor->GetEditorWorldContext().World();
	TSharedPtr<FLevelCollectionModel> Model = SharedWorldModel(EditorWorld);
	if (Model.IsValid())
	{
		FLevelModelList ModelList = Model->GetFilteredLevels();
		for (TSharedPtr<FLevelModel> LevelModel : ModelList)
		{
			FUIAction Action(FExecuteAction::CreateRaw(this, &FWorldBrowserModule::SetCurrentSublevel, LevelModel),
				FCanExecuteAction(),
				FIsActionChecked::CreateRaw(this, &FWorldBrowserModule::IsCurrentSublevel, LevelModel));
			MenuBuilder.AddMenuEntry(FText::FromString(LevelModel->GetDisplayName()), FText::GetEmpty(), FSlateIcon(), Action, NAME_None, EUserInterfaceActionType::Button);
		}
	}
}

bool FWorldBrowserModule::IsCurrentSublevel(TSharedPtr<FLevelModel> InLevelModel)
{
	return InLevelModel->IsCurrent();
}

void FWorldBrowserModule::SetCurrentSublevel(TSharedPtr<FLevelModel> InLevelModel)
{
	InLevelModel->MakeLevelCurrent();
}

void FWorldBrowserModule::StartupModule()
{
	FLevelCollectionCommands::Register();

	// register the editor mode
	FEditorModeRegistry::Get().RegisterMode<FStreamingLevelEdMode>(
		FBuiltinEditorModes::EM_StreamingLevel,
		NSLOCTEXT("WorldBrowser", "StreamingLevelMode", "Level Transform Editing"));

	if (ensure(GEngine))
	{
		GEngine->OnWorldAdded().AddRaw(this, &FWorldBrowserModule::OnWorldCreated);
		GEngine->OnWorldDestroyed().AddRaw(this, &FWorldBrowserModule::OnWorldDestroyed);
	}

	UWorldComposition::WorldCompositionChangedEvent.AddRaw(this, &FWorldBrowserModule::OnWorldCompositionChanged);

	// Have to check GIsEditor because right now editor modules can be loaded by the game
	// Once LoadModule is guaranteed to return NULL for editor modules in game, this can be removed
	// Without this check, loading the level editor in the game will crash
	if (GIsEditor)
	{
		// Extend the level viewport levels menu
		FLevelEditorModule& LevelEditorModule = FModuleManager::Get().LoadModuleChecked<FLevelEditorModule>("LevelEditor");
		LevelMenuExtender = FLevelEditorModule::FLevelEditorMenuExtender::CreateRaw(this, &FWorldBrowserModule::BindLevelMenu);
		auto& MenuExtenders = LevelEditorModule.GetAllLevelEditorLevelMenuExtenders();
		MenuExtenders.Add(LevelMenuExtender);
		LevelMenuExtenderHandle = MenuExtenders.Last().GetHandle();
	}

	FLevelFolders::Init();
}

void FWorldBrowserModule::ShutdownModule()
{
	OnBrowseWorld.Broadcast(nullptr);
	
	ReleaseWorldModel();
    ShutdownDelegate.Broadcast(); 

	FLevelFolders::Cleanup();

	if (GEngine)
	{
		GEngine->OnWorldAdded().RemoveAll(this);
		GEngine->OnWorldDestroyed().RemoveAll(this);
	}

	UWorldComposition::WorldCompositionChangedEvent.RemoveAll(this);
	
	FLevelCollectionCommands::Unregister();

	// unregister the editor mode
	FEditorModeRegistry::Get().UnregisterMode(FBuiltinEditorModes::EM_StreamingLevel);
}

void FWorldBrowserModule::ReleaseWorldModel()
{
	if (WorldModel)
	{
		// So we have to be the last owner of this model
		check(WorldModel.IsUnique());
		WorldModel.Reset();
	}
}

TSharedRef<SWidget> FWorldBrowserModule::CreateWorldBrowserHierarchy()
{
	return CreateWorldBrowserHierarchyWidget()->GetWidget();
}

TSharedRef<UE::WorldHierarchy::IWorldHierarchy> FWorldBrowserModule::CreateWorldBrowserHierarchyWidget()
{
	UWorld* EditorWorld = GEditor->GetEditorWorldContext().World();
	return SNew(SWorldHierarchy).InWorld(EditorWorld);
}

TSharedRef<SWidget> FWorldBrowserModule::CreateWorldBrowserDetails()
{
	UWorld* EditorWorld = GEditor->GetEditorWorldContext().World();
	return SNew(SWorldDetails).InWorld(EditorWorld);
}

TSharedRef<SWidget> FWorldBrowserModule::CreateWorldBrowserComposition()
{
	UWorld* EditorWorld = GEditor->GetEditorWorldContext().World();
	return SNew(SWorldComposition).InWorld(EditorWorld);
}

void FWorldBrowserModule::OnWorldCreated(UWorld* InWorld)
{
	if (InWorld && 
		InWorld->WorldType == EWorldType::Editor)
	{
		OnBrowseWorld.Broadcast(InWorld);
	}
}

void FWorldBrowserModule::OnWorldCompositionChanged(UWorld* InWorld)
{
	if (InWorld && 
		InWorld->WorldType == EWorldType::Editor)
	{
		OnBrowseWorld.Broadcast(NULL);
		// Release World Model here so that a new one gets created in ShareWorldModel even if the World is the same.
		ReleaseWorldModel();
		OnBrowseWorld.Broadcast(InWorld);
	}
}

void FWorldBrowserModule::OnWorldDestroyed(UWorld* InWorld)
{
	// Is there any editors alive?
	if (WorldModel.IsValid())
	{
		UWorld* ManagedWorld = WorldModel->GetWorld(/*bEvenIfPendingKill*/true);
		// Is it our world gets cleaned up?
		if (ManagedWorld == InWorld)
		{
			// Will reset all references to a shared world model
			OnBrowseWorld.Broadcast(NULL);
			ReleaseWorldModel();
		}
	}
}

TSharedPtr<FLevelCollectionModel> FWorldBrowserModule::SharedWorldModel(UWorld* InWorld)
{
	if (!WorldModel.IsValid() || WorldModel->GetWorld() != InWorld)
	{
		if (InWorld)
		{
			if (InWorld->WorldComposition)
			{
				WorldModel = FWorldTileCollectionModel::Create(InWorld);
			}
			else
			{
				WorldModel = FStreamingLevelCollectionModel::Create(InWorld);
			}
		}
	}
	
	return WorldModel;
}

#undef LOCTEXT_NAMESPACE
