// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/MaxSizeof.h"
#include "UObject/ObjectKey.h"

namespace WorldHierarchy
{
	/** Variable type for defining identifies for world tree items */
	struct FWorldTreeItemID
	{
	public:

		enum class EType : uint8 
		{
			Object,
			Folder,
			MissingObject,
			Null,
		};

		FWorldTreeItemID() : Type(EType::Null), CachedHash(0) {}

		/** IDs for UObjects */
		FWorldTreeItemID(const UObject* InObject, FName ItemName) : Type(EType::Object)
		{
			if (InObject != nullptr)
			{
				new (Data) FObjectKey(InObject);
				CachedHash = CalculateTypeHash();
			}
			else
			{
				CreateAsMissing(ItemName);
			}
		}
		FWorldTreeItemID(const FObjectKey& InKey) : Type(EType::Object)
		{
			new (Data) FObjectKey(InKey);
			CachedHash = CalculateTypeHash();
		}

		/** IDs for folders */
		FWorldTreeItemID(const FName& InFolder) : Type(EType::Folder)
		{
			new (Data) FName(InFolder);
			CachedHash = CalculateTypeHash();
		}

		/** Copy construction/assignment */
		FWorldTreeItemID(const FWorldTreeItemID& Other)
		{
			*this = Other;
		}
		FWorldTreeItemID& operator=(const FWorldTreeItemID& Other)
		{
			Type = Other.Type;
			switch (Type)
			{
			case EType::Object:
				new (Data) FObjectKey(Other.GetAsObjectKey());
				break;
			case EType::Folder:
			case EType::MissingObject:
				new (Data) FName(Other.GetAsNameRef());
				break;
			default:
				break;
			}

			CachedHash = CalculateTypeHash();
			return *this;
		}

		/** Move construction/assignment */
		FWorldTreeItemID(FWorldTreeItemID&& Other)
		{
			*this = MoveTemp(Other);
		}
		FWorldTreeItemID& operator=(FWorldTreeItemID&& Other)
		{
			Swap(*this, Other);
			return *this;
		}

		~FWorldTreeItemID()
		{
			switch (Type)
			{
			case EType::Object:
				GetAsObjectKey().~FObjectKey();
				break;
			case EType::Folder:
			case EType::MissingObject:
				GetAsNameRef().~FName();
				break;
			default:
				break;
			}
		}

		friend bool operator==(const FWorldTreeItemID& One, const FWorldTreeItemID& Other)
		{
			return One.Compare(Other);
		}

		friend bool operator!=(const FWorldTreeItemID& One, const FWorldTreeItemID& Other)
		{
			return !One.Compare(Other);
		}

		uint32 CalculateTypeHash() const
		{
			uint32 Hash = 0;
			switch (Type)
			{
			case EType::Object:
				Hash = GetTypeHash(GetAsObjectKey());
				break;
			case EType::Folder:
			case EType::MissingObject:
				Hash = GetTypeHash(GetAsNameRef());
				break;
			default:
				break;
			}

			return HashCombine((uint8)Type, Hash);
		}

		friend uint32 GetTypeHash(const FWorldTreeItemID& ItemID)
		{
			return ItemID.CachedHash;
		}

	private:

		void CreateAsMissing(FName ObjectName)
		{
			Type = EType::MissingObject;
			new (Data) FName(ObjectName);
			CachedHash = CalculateTypeHash();
		}

		FObjectKey& GetAsObjectKey() const { return *reinterpret_cast<FObjectKey*>(Data); }
		FName& GetAsNameRef() const { return *reinterpret_cast<FName*>(Data); }

		/** Compares the specified ID with this one, and returns true if they match. */
		bool Compare(const FWorldTreeItemID& Other) const
		{
			bool bSame = false;
			
			if (Type == Other.Type && CachedHash == Other.CachedHash)
			{
				switch (Type)
				{
				case EType::Object:
					bSame = GetAsObjectKey() == Other.GetAsObjectKey();
					break;
				case EType::Folder:
				case EType::MissingObject:
					bSame = GetAsNameRef() == Other.GetAsNameRef();
					break;
				case EType::Null:
					bSame = true;
					break;
				default:
					check(false);
					break;
				}
			}

			return bSame;
		}

	private:

		static const uint32 MaxDataSize = TMaxSizeof<FObjectKey, FName>::Value;

		EType Type;
		uint32 CachedHash;
		mutable uint8 Data[MaxDataSize];
	};

}	// namespace WorldHierarchy