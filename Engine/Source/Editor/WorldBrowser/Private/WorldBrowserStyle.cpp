// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldBrowserStyle.h"

#include "Styling/SlateStyleMacros.h"

namespace WorldBrowser
{
	FWorldBrowserStyle::FWorldBrowserStyle()
		: FSlateStyleSet("WorldBrowserStyle")
	{
		const FVector2D Icon16x16(16.0f, 16.0f);
		SetContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));
		SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));
		
		Set("WorldBrowser.VisibleInGame", new CORE_IMAGE_BRUSH_SVG("Starship/Common/VisibleInGame", Icon16x16));
		Set("WorldBrowser.HiddenInGame", new CORE_IMAGE_BRUSH_SVG("Starship/Common/HiddenInGame", Icon16x16));
		
		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}
}
