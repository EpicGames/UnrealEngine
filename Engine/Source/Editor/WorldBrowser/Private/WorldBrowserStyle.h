// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateStyle.h"

namespace WorldBrowser
{
	class FWorldBrowserStyle final : public FSlateStyleSet
	{
	public:

		FWorldBrowserStyle();
		virtual ~FWorldBrowserStyle() { FSlateStyleRegistry::UnRegisterSlateStyle(*this); }
	
		static FWorldBrowserStyle& Get()
		{
			static FWorldBrowserStyle Inst;
			return Inst;
		}
	};
}

