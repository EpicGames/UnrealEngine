// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldHierarchyColumns.h"

namespace UE::WorldHierarchy::HierarchyColumns
{
	const FName ColumnID_LevelLabel( "Level" );
	const FName ColumnID_EditorVisibility( "EditorVisibility" );
	const FName ColumnID_GameVisibility( "GameVisibility" );
	const FName ColumnID_LightingScenario( "LightingScenario" );
	const FName ColumnID_Lock( "Lock" );
	const FName ColumnID_SCCStatus( "SCC_Status" );
	const FName ColumnID_Save( "Save" );
	const FName ColumnID_Color("Color");
	const FName ColumnID_Kismet( "Blueprint" );
}
