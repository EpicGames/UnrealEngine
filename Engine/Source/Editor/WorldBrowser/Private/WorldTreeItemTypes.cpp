// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldTreeItemTypes.h"

#include "Editor.h"
#include "Templates/Casts.h"
#include "LevelModel.h"
#include "LevelCollectionModel.h"
#include "Misc/PackageName.h"
#include "Engine/Engine.h"
#include "SWorldHierarchyImpl.h"
#include "ToolMenus.h"
#include "Styling/AppStyle.h"
#include "AssetSelection.h"
#include "SWorldHierarchyImpl.h"

#include "Engine/LevelStreamingAlwaysLoaded.h"
#include "Engine/LevelStreamingDynamic.h"

#define LOCTEXT_NAMESPACE "WorldBrowser"

namespace WorldHierarchy
{
	/** Utility function to get the leaf name of a specified path */
	FName GetPathLeafName(const FName& InPath)
	{
		FString PathString = InPath.ToString();
		FName LeafName = InPath;
		int32 LeafIndex = 0;
		if (PathString.FindLastChar(TCHAR('/'), LeafIndex))
		{
			LeafName = FName(*PathString.RightChop(LeafIndex + 1));
		}
		return LeafName;
	}

	/** Utility function to get world assets from a drag operation */
	bool GetWorldAssetsFromDrag(TSharedPtr<FDragDropOperation> DragOp, TArray<FAssetData>& OutWorlds)
	{
		TArray<FAssetData> AssetList = AssetUtil::ExtractAssetDataFromDrag(DragOp);
		
		for (const FAssetData& Asset : AssetList)
		{
			if (Asset.AssetClassPath == UWorld::StaticClass()->GetClassPathName())
			{
				OutWorlds.Add(Asset);
			}
		}

		return OutWorlds.Num() > 0;
	}
	
	bool IsInPie()
	{
		return GEditor && GEditor->GetPlayInEditorSessionInfo().IsSet();
	}

	//------------------------
	// FLevelModelTreeItem
	//------------------------


	FLevelModelTreeItem::FLevelModelTreeItem(TSharedRef<FLevelModel> InLevelModel)
		: LevelModel(InLevelModel)
		, bPersistentLevel(InLevelModel->IsPersistent())
		, ID(InLevelModel->GetLevelObject(), *InLevelModel->GetDisplayName())
	{
		Flags.bExpanded = InLevelModel->GetLevelExpansionFlag();
	}

	FLevelModelList FLevelModelTreeItem::GetModel() const
	{
		FLevelModelList Models;
		if (LevelModel.IsValid())
		{
			Models.Add(LevelModel.Pin());
		}

		return Models;
	}

	FWorldTreeItemID FLevelModelTreeItem::GetID() const
	{
		return ID;
	}

	FWorldTreeItemPtr FLevelModelTreeItem::CreateParent() const
	{
		FWorldTreeItemPtr ParentPtr;
		TSharedPtr<FLevelModel> Model = LevelModel.Pin();

		if (Model->GetFolderPath() != NAME_None)
		{
			ParentPtr = MakeShareable(new FFolderTreeItem(Model->GetFolderPath()));
			ParentPtr->WorldModel = WorldModel;
		}
		else if (Model->GetParent().IsValid())
		{
			ParentPtr = MakeShareable(new FLevelModelTreeItem(Model->GetParent().ToSharedRef()));
			ParentPtr->WorldModel = WorldModel;
		}

		return ParentPtr;
	}

	FString FLevelModelTreeItem::GetDisplayString() const
	{
		return LevelModel.IsValid() ? LevelModel.Pin()->GetDisplayName() : FString();
	}

	FText FLevelModelTreeItem::GetToolTipText() const
	{
		FString PackageName = LevelModel.IsValid() ? LevelModel.Pin()->GetLongPackageName().ToString() : FString();
		if (FPackageName::DoesPackageExist(PackageName))
		{
			return FText::FromString(PackageName);
		}
		else
		{
			return LOCTEXT("UnsavedLevel", "Unsaved Level");
		}
	}

	FText FLevelModelTreeItem::GetLockToolTipText() const
	{
		FText LockToolTip;

		if (GEngine && GEngine->bLockReadOnlyLevels)
		{
			if (LevelModel.IsValid() && LevelModel.Pin()->IsFileReadOnly())
			{
				LockToolTip = LOCTEXT("ReadOnly_LockButtonToolTip", "Read-Only levels are locked!");
			}
		}

		LockToolTip = LOCTEXT("LockButtonToolTip", "Toggle Level Lock");

		return LockToolTip;
	}

	FText FLevelModelTreeItem::GetEditorVisibilityToolTipText() const
	{
		return LOCTEXT("EditorVisibilityButtonToolTip", "Toggle whether Level is visible in the editor");
	}

	FText FLevelModelTreeItem::GetGameVisibilityToolTipText() const
	{
		if (IsPersistentLevel())
		{
			return LOCTEXT("LevelModel.LevelGameVisibilityButtonToolTip.PersistentLevel", "Game visibility cannot be toggled for persistent level.");
		}

		if (IsInPie())
		{
			return LOCTEXT("LevelModel.LevelGameVisibilityButtonToolTip.Pie", "Game visibility cannot be toggled during PIE.");
		}
		
		return LOCTEXT("LevelModel.GameVisibilityButtonToolTip.Normal", "Toggle whether Level is visible in-game");
	}

	FText FLevelModelTreeItem::GetSaveToolTipText() const
	{
		return LOCTEXT("SaveButtonToolTip", "Save Level");
	}

	FString FLevelModelTreeItem::GetPackageFileName() const
	{
		return IsLoaded() ? LevelModel.Pin()->GetPackageFileName() : FString();
	}

	FWorldTreeItemID FLevelModelTreeItem::GetParentID() const
	{
		FWorldTreeItemID ParentID;
		TSharedPtr<FLevelModel> Model = LevelModel.Pin();

		if (Parent.IsValid())
		{
			ParentID = Parent->GetID();
		}
		else if (!Model->GetFolderPath().IsNone())
		{
			ParentID = Model->GetFolderPath();
		}
		else if (Model->GetParent().IsValid())
		{
			ParentID = FWorldTreeItemID(Model->GetParent()->GetLevelObject(), *Model->GetParent()->GetDisplayName());
		}

		return ParentID;
	}

	bool FLevelModelTreeItem::CanHaveChildren() const
	{
		TSharedPtr<FLevelModel> Model = LevelModel.Pin();
		return Model.IsValid() && !Model->GetParent().IsValid();
	}

	void FLevelModelTreeItem::SetParentPath(const FName& InParentPath)
	{
		if (LevelModel.IsValid())
		{
			LevelModel.Pin()->SetFolderPath(InParentPath);
		}
	}

	void FLevelModelTreeItem::SetExpansion(bool bExpanded)
	{
		IWorldTreeItem::SetExpansion(bExpanded);

		if (LevelModel.IsValid())
		{
			LevelModel.Pin()->SetLevelExpansionFlag(bExpanded);
		}
	}

	bool FLevelModelTreeItem::HasModel(TSharedPtr<FLevelModel> InLevelModel) const
	{
		return InLevelModel.IsValid() && LevelModel.Pin().Get() == InLevelModel.Get();
	}

	TSet<FName> FLevelModelTreeItem::GetAncestorPaths() const
	{
		TSet<FName> Ancestors;

		if (LevelModel.IsValid())
		{
			FName CurrentPath = LevelModel.Pin()->GetFolderPath();

			while (!CurrentPath.IsNone())
			{
				Ancestors.Add(CurrentPath);
				CurrentPath = GetParentPath(CurrentPath);
			}
		}

		return Ancestors;
	}

	bool FLevelModelTreeItem::CanSave() const
	{
		return IsLoaded();
	}

	bool FLevelModelTreeItem::HasLightingControls() const
	{
		return IsLoaded();
	}

	bool FLevelModelTreeItem::HasLockControls() const
	{
		return IsLoaded() ;
	}

	bool FLevelModelTreeItem::HasEditorVisibilityControls() const
	{
		return IsLoaded();
	}

	bool FLevelModelTreeItem::HasGameVisibilityControls() const
	{
		return HasEditorVisibilityControls() 
			// Unreal generally does not allow hiding the persistent level (root level) - so no controls if this item is the persistent level. 
			&& !IsPersistentLevel()
			&& !IsInPie();
	}

	bool FLevelModelTreeItem::HasColorButtonControls() const
	{
		// The root level does not have a color button
		return IsLoaded() && Parent.IsValid();
	}

	bool FLevelModelTreeItem::HasKismet() const
	{
		return LevelModel.IsValid() && LevelModel.Pin()->HasKismet();
	}

	bool FLevelModelTreeItem::IsCurrent() const
	{
		return LevelModel.IsValid() && LevelModel.Pin()->IsCurrent();
	}

	void FLevelModelTreeItem::MakeCurrent()
	{
		if (LevelModel.IsValid())
		{
			LevelModel.Pin()->MakeLevelCurrent();
		}
	}

	bool FLevelModelTreeItem::HasValidPackage() const
	{
		return LevelModel.IsValid() && LevelModel.Pin()->HasValidPackage();
	}

	bool FLevelModelTreeItem::IsDirty() const
	{
		return LevelModel.IsValid() && LevelModel.Pin()->IsDirty();
	}

	bool FLevelModelTreeItem::IsLoaded() const
	{
		return LevelModel.IsValid() && LevelModel.Pin()->IsLoaded();
	}

	FLinearColor FLevelModelTreeItem::GetDrawColor() const
	{
		return LevelModel.IsValid() ? LevelModel.Pin()->GetLevelColor() : FLinearColor::White;
	}

	void FLevelModelTreeItem::SetDrawColor(const FLinearColor& Color)
	{
		if (LevelModel.IsValid())
		{
			LevelModel.Pin()->SetLevelColor(Color);
		}
	}

	bool FLevelModelTreeItem::IsVisibleInEditor() const
	{
		return LevelModel.IsValid() ? LevelModel.Pin()->IsVisibleInEditor() : false;
	}

	bool FLevelModelTreeItem::IsVisibleInGame() const
	{
		return LevelModel.IsValid() ? LevelModel.Pin()->IsVisibleInGame() : false;
	}

	void FLevelModelTreeItem::OnToggleEditorVisibility()
	{
		if (LevelModel.IsValid())
		{
			SetVisibleInEditor(!LevelModel.Pin()->IsVisibleInEditor());
		}
	}

	void FLevelModelTreeItem::OnShowInEditorOnlySelected()
	{
		SetSelectedLevelsToPopulatedList();
		WorldModel.Pin()->ShowInEditorOnlySelectedLevels();
	}

	void FLevelModelTreeItem::OnShowInEditorAllButSelected()
	{
		SetSelectedLevelsToPopulatedList();
		WorldModel.Pin()->ShowInEditorAllButSelectedLevels();
	}

	void FLevelModelTreeItem::OnToggleGameVisibility()
	{
		if (LevelModel.IsValid())
		{
			SetVisibleInGame(!LevelModel.Pin()->IsVisibleInGame());
		}
	}

	void FLevelModelTreeItem::OnShowInGameOnlySelected()
	{
		SetSelectedLevelsToPopulatedList();
		WorldModel.Pin()->ShowInGameOnlySelectedLevels();
	}

	void FLevelModelTreeItem::OnShowInGameAllButSelected()
	{
		SetSelectedLevelsToPopulatedList();
		WorldModel.Pin()->ShowInGameAllButSelectedLevels();
	}

	void FLevelModelTreeItem::PopulateLevelModelList(FLevelModelList& InModelList)
	{
		if (LevelModel.IsValid())
		{
			InModelList.Add(LevelModel.Pin());
		}
	}

	void FLevelModelTreeItem::SetVisibleInEditor(bool bVisible)
	{
		FLevelModelList LevelModels;

		if (LevelModel.IsValid())
		{
			LevelModels.Add(LevelModel.Pin());
		}

		if (bVisible)
		{
			WorldModel.Pin()->ShowLevelsInEditor(LevelModels);
		}
		else
		{
			WorldModel.Pin()->HideLevelsInEditor(LevelModels);
		}
	}

	void FLevelModelTreeItem::SetVisibleInGame(bool bVisible)
	{
		FLevelModelList LevelModels;

		if (LevelModel.IsValid())
		{
			LevelModels.Add(LevelModel.Pin());
		}

		if (bVisible)
		{
			WorldModel.Pin()->ShowLevelsInGame(LevelModels);
		}
		else
		{
			WorldModel.Pin()->HideLevelsInGame(LevelModels);
		}
	}

	void FLevelModelTreeItem::OnToggleLightingScenario()
	{
		TSharedPtr<FLevelModel> Model = LevelModel.Pin();
		if (Model.IsValid())
		{
			Model->SetIsLightingScenario(!Model->IsLightingScenario());
		}
	}

	void FLevelModelTreeItem::OnToggleLock()
	{
		if (LevelModel.IsValid())
		{
			SetLocked(!LevelModel.Pin()->IsLocked());
		}
	}

	void FLevelModelTreeItem::OnLockOnlySelected()
	{
		SetSelectedLevelsToPopulatedList();
		WorldModel.Pin()->LockOnlySelectedLevels();
	}

	void FLevelModelTreeItem::OnLockAllButSelected()
	{
		SetSelectedLevelsToPopulatedList();
		WorldModel.Pin()->LockAllButSelectedLevels();
	}

	bool FLevelModelTreeItem::IsLocked() const
	{
		return IsLoaded() && LevelModel.Pin()->IsLocked();
	}

	bool FLevelModelTreeItem::IsReadOnly() const
	{
		return IsLoaded() && LevelModel.Pin()->IsFileReadOnly();
	}

	bool FLevelModelTreeItem::IsTransient() const
	{
		return (LevelModel.IsValid() && LevelModel.Pin()->IsTransient());
	}

	void FLevelModelTreeItem::SetLocked(bool bLocked)
	{
		FLevelModelList LevelModels;

		if (LevelModel.IsValid())
		{
			LevelModels.Add(LevelModel.Pin());
		}

		if (bLocked)
		{
			WorldModel.Pin()->LockLevels(LevelModels);
		}
		else
		{
			WorldModel.Pin()->UnlockLevels(LevelModels);
		}
	}

	void FLevelModelTreeItem::OnSave()
	{
		FLevelModelList LevelModels;

		if (LevelModel.IsValid())
		{
			LevelModels.Add(LevelModel.Pin());
		}

		WorldModel.Pin()->SaveLevels(LevelModels);
	}

	void FLevelModelTreeItem::OnOpenKismet()
	{
		if (LevelModel.IsValid())
		{
			LevelModel.Pin()->OpenKismet();
		}
	}

	bool FLevelModelTreeItem::GetLevelSelectionFlag() const
	{
		return LevelModel.IsValid() && LevelModel.Pin()->GetLevelSelectionFlag();
	}

	bool FLevelModelTreeItem::IsLightingScenario() const
	{
		return LevelModel.IsValid() && LevelModel.Pin()->IsLightingScenario();
	}

	const FSlateBrush* FLevelModelTreeItem::GetHierarchyItemBrush() const
	{
		UClass* StreamingClass = LevelModel.IsValid() ? LevelModel.Pin()->GetStreamingClass() : nullptr;
		
		if (StreamingClass == ULevelStreamingDynamic::StaticClass())
		{
			return FAppStyle::GetBrush("WorldBrowser.LevelStreamingBlueprint");
		}

		if (StreamingClass == ULevelStreamingAlwaysLoaded::StaticClass())
		{
			return FAppStyle::GetBrush("WorldBrowser.LevelStreamingAlwaysLoaded");
		}

		return nullptr;
	}

	bool FLevelModelTreeItem::CanChangeParents() const
	{
		return Parent.IsValid() && LevelModel.IsValid() && LevelModel.Pin()->IsUserManaged();
	}

	void FLevelModelTreeItem::GenerateContextMenu(UToolMenu* Menu, const SWorldHierarchyImpl& Hierarchy)
	{
		if (!Parent.IsValid() && WorldModel.Pin()->HasFolderSupport())
		{
			// Persistent level items should be able to create new folders beneath them in the hierarchy
			const FSlateIcon NewFolderIcon(FAppStyle::GetAppStyleSetName(), "WorldBrowser.NewFolderIcon");

			FName RootPath = LevelModel.Pin()->GetFolderPath();
			auto NewFolderAction = FExecuteAction::CreateSP(const_cast<SWorldHierarchyImpl*>(&Hierarchy), &SWorldHierarchyImpl::CreateFolder, LevelModel.Pin(), RootPath, /*bMoveSelection*/ false);

			FToolMenuSection& Section = Menu->AddSection("Section");
			Section.AddMenuEntry("CreateFolder", LOCTEXT("CreateFolder", "Create Folder"), FText(), NewFolderIcon, FUIAction(NewFolderAction));
		}
	}

	FValidationInfo FLevelModelTreeItem::ValidateDrop(const FDragDropEvent& DragEvent) const
	{
		FValidationInfo ValidationInfo;

		TSharedPtr<FWorldBrowserDragDropOp> HierarchyOp = DragEvent.GetOperationAs<FWorldBrowserDragDropOp>();

		if (HierarchyOp.IsValid())
		{
			TArray<FWorldTreeItemPtr> SelectedItems = HierarchyOp->GetDraggedItems();

			if (SelectedItems.Contains(AsShared()))
			{
				ValidationInfo.ValidationText = LOCTEXT("DropInvalid_CannotAttachToSelf", "Selection cannot be attached to itself");
				ValidationInfo.bValid = false;
			}
			else
			{
				TSet<FName> Ancestors = GetAncestorPaths();

				for (FWorldTreeItemPtr Item : SelectedItems)
				{
					FWorldTreeItemPtr ItemParent = Item->GetParent();

					if (!Item->CanChangeParents())
					{
						ValidationInfo.ValidationText = LOCTEXT("DropInvalid_ItemCannotMove", "Cannot move selection");
						ValidationInfo.bValid = false;
					}
					else if (ItemParent.IsValid() && ItemParent.Get() == this)
					{
						ValidationInfo.ValidationText = LOCTEXT("DropInvalid_ItemAlreadyAttached", "Selection is already attached to this item");
						ValidationInfo.bValid = false;
					}
					else if (FFolderTreeItem* Folder = Item->GetAsFolderTreeItem())
					{
						if (Ancestors.Contains(Folder->GetFullPath()))
						{
							ValidationInfo.ValidationText = LOCTEXT("DropInvalid_CannotBeChildOfSelf", "Selection cannot become a child of itself");
							ValidationInfo.bValid = false;
						}
					}

					if (!ValidationInfo.bValid)
					{
						break;
					}
				}
			}
		}
		else
		{
			TArray<FAssetData> WorldAssets;
			ValidationInfo.bValid = GetWorldAssetsFromDrag(DragEvent.GetOperation(), WorldAssets);
		}

		if (ValidationInfo.bValid && ValidationInfo.ValidationText.IsEmpty())
		{
			FString ModelString = LevelModel.IsValid() ? LevelModel.Pin()->GetDisplayName() : TEXT("level");
			ValidationInfo.ValidationText = FText::Format(LOCTEXT("DropValid_MoveSelectionToLevel", "Drop selection on {0}"), FText::FromString(ModelString));
		}

		return ValidationInfo;
	}

	void FLevelModelTreeItem::OnDrop(const FDragDropEvent& DragEvent, TSharedRef<SWorldHierarchyImpl> Hierarchy)
	{
		TSharedPtr<FWorldBrowserDragDropOp> HierarchyOp = DragEvent.GetOperationAs<FWorldBrowserDragDropOp>();

		if (HierarchyOp.IsValid() && LevelModel.IsValid())
		{
			Hierarchy->MoveDroppedItems(HierarchyOp->GetDraggedItems(), LevelModel.Pin()->GetFolderPath());
			LevelModel.Pin()->OnDrop(DragEvent.GetOperationAs<FLevelDragDropOp>());
		}
		else
		{
			TArray<FAssetData> Worlds;
			if (GetWorldAssetsFromDrag(DragEvent.GetOperation(), Worlds))
			{
				Hierarchy->AddDroppedLevelsToFolder(Worlds, NAME_None);
			}
		}
	}
	
	void FLevelModelTreeItem::SetSelectedLevelsToPopulatedList()
	{
		FLevelModelList LevelsToChange;
		PopulateLevelModelList(LevelsToChange);
		if (GetLevelSelectionFlag())
		{
			FLevelModelList CurrentLevels = WorldModel.Pin()->GetSelectedLevels();
			LevelsToChange.Append(CurrentLevels);
		}
		WorldModel.Pin()->SetSelectedLevels(LevelsToChange);
	}

	bool FLevelModelTreeItem::IsPersistentLevel() const
	{
		const TSharedPtr<FLevelModel> LevelModelPin = LevelModel.Pin();
		return LevelModelPin && LevelModelPin->IsPersistent();
	}


	//------------------------
	// FFolderTreeItem
	//------------------------

	FFolderTreeItem::FFolderTreeItem(FName InPath)
		: Path(InPath)
		, LeafName(GetPathLeafName(InPath))
	{
	}

	FWorldTreeItemID FFolderTreeItem::GetID() const
	{
		return FWorldTreeItemID(Path);
	}

	FWorldTreeItemPtr FFolderTreeItem::CreateParent() const
	{
		FWorldTreeItemPtr ParentPtr;

		// Assume that the parent of this folder is another folder first
		const FName ParentPath(GetParentPath(Path));
		if ( !ParentPath.IsNone() )
		{
			ParentPtr = MakeShareable(new FFolderTreeItem(ParentPath));
			ParentPtr->WorldModel = WorldModel;
		}

		// If there's no parent path, assume that it belongs to root level 0
		if ( !ParentPtr.IsValid() && WorldModel.IsValid() )
		{
			FLevelModelList& RootLevels = WorldModel.Pin()->GetRootLevelList();

			if (RootLevels.Num() > 0)
			{
				// TODO: Find a better solution than just root 0?
				ParentPtr = MakeShareable((new FLevelModelTreeItem(RootLevels[0].ToSharedRef())));
				ParentPtr->WorldModel = WorldModel;
			}
		}

		return ParentPtr;
	}

	FWorldTreeItemID FFolderTreeItem::GetParentID() const
	{
		FWorldTreeItemID ParentID;

		if (Parent.IsValid())
		{
			ParentID = Parent->GetID();
		}
		else if (!GetParentPath(Path).IsNone())
		{
			ParentID = GetParentPath(Path);
		}
		else if (WorldModel.IsValid())
		{
			FLevelModelList& RootLevels = WorldModel.Pin()->GetRootLevelList();

			if (RootLevels.Num() > 0)
			{
				// TODO: Find a better solution than just root 0?
				TSharedPtr<FLevelModel> Root = RootLevels[0];
				ParentID = FWorldTreeItemID(Root->GetLevelObject(), *Root->GetDisplayName());
			}
		}

		return ParentID;
	}

	TSet<FName> FFolderTreeItem::GetAncestorPaths() const
	{
		TSet<FName> Ancestors;

		FName CurrentPath = GetParentPath(GetFullPath());

		while (!CurrentPath.IsNone())
		{
			Ancestors.Add(CurrentPath);
			CurrentPath = GetParentPath(CurrentPath);
		}

		return Ancestors;
	}

	FString FFolderTreeItem::GetDisplayString() const
	{
		return LeafName.ToString();
	}

	FText FFolderTreeItem::GetToolTipText() const
	{
		return FText::FromName(Path);
	}

	FText FFolderTreeItem::GetLockToolTipText() const
	{
		FText LockToolTip;

		if (IsLocked())
		{
			LockToolTip = LOCTEXT("FolderUnlockAllLevels_ToolTip", "Unlock All Levels");
		}
		else
		{
			LockToolTip = LOCTEXT("FolderLockAllLevels_ToolTip", "Lock All Levels");
		}

		return LockToolTip;
	}

	FText FFolderTreeItem::GetEditorVisibilityToolTipText() const
	{
		return LOCTEXT("FolderEditorVisibilityButtonToolTip", "Toggle Editor Visibility for All Levels");
	}

	FText FFolderTreeItem::GetGameVisibilityToolTipText() const
	{
		if (IsInPie())
		{
			return LOCTEXT("FolderModel.GameVisibilityButtonToolTip.Pie", "Game visibility cannot be toggled during PIE.");
		}
		
		return LOCTEXT("FolderModel.FolderGameVisibilityButtonToolTip", "Toggle Game Visibility for All Levels");
	}

	FText FFolderTreeItem::GetSaveToolTipText() const
	{
		return LOCTEXT("FolderSaveButtonToolTip", "Save All Levels");
	}

	void FFolderTreeItem::SetNewPath(FName NewPath)
	{
		Path = NewPath;
		LeafName = GetPathLeafName(NewPath);
	}

	void FFolderTreeItem::SetParentPath(const FName& InParentPath)
	{
		Path = FName(*(InParentPath.ToString() + LeafName.ToString()));
	}

	bool FFolderTreeItem::CanSave() const
	{
		return IsAnyChildLoaded();
	}

	bool FFolderTreeItem::HasLockControls() const
	{
		// If the folder has no level models associated with it, always show the lock icon
		return GetLevelModels().Num() == 0 || IsAnyChildLoaded();
	}

	bool FFolderTreeItem::HasEditorVisibilityControls() const
	{
		// If the folder has no level models associated with it, always show the visibility icon
		return GetLevelModels().Num() == 0 || IsAnyChildLoaded();
	}

	bool FFolderTreeItem::HasGameVisibilityControls() const
	{
		return HasEditorVisibilityControls()
			&& !IsInPie();
	}

	bool FFolderTreeItem::HasValidPackage() const
	{
		// Return true only if all children have valid packages
		bool bAllValid = true;
		for (const auto& Child : Children)
		{
			bAllValid = Child->HasValidPackage();

			if (!bAllValid)
			{
				break;
			}
		}

		return bAllValid;
	}

	bool FFolderTreeItem::IsDirty() const
	{
		FLevelModelList LevelModels = GetLevelModels();
		
		return LevelModels.ContainsByPredicate([](const TSharedPtr<FLevelModel>& LevelModel) -> bool
		{
			return LevelModel.IsValid() && LevelModel->IsDirty();
		});
	}

	bool FFolderTreeItem::IsAnyChildLoaded() const
	{
		FLevelModelList LevelModels = GetLevelModels();

		return LevelModels.ContainsByPredicate([](const TSharedPtr<FLevelModel>& LevelModel) -> bool
		{
			return LevelModel.IsValid() && LevelModel->IsLoaded();
		});
	}

	FLevelModelList FFolderTreeItem::SetSelectionToFolderChildren()
	{
		// This can be triggered on the non selected folder, so get its children instead
		FLevelModelList LevelsToChange;
		for (auto& Child : Children)
		{
			Child->PopulateLevelModelList(LevelsToChange);
		}
		if (GetLevelSelectionFlag())
		{
			FLevelModelList CurrentLevels = WorldModel.Pin()->GetSelectedLevels();
			LevelsToChange.Append(CurrentLevels);
		}
		WorldModel.Pin()->SetSelectedLevels(LevelsToChange);
		return LevelsToChange;
	}

	bool FFolderTreeItem::IsVisibleInEditor() const
	{
		for (FWorldTreeItemPtr Child : Children)
		{
			if (Child->IsVisibleInEditor())
			{
				return true;
			}
		}

		return false;
	}

	bool FFolderTreeItem::IsVisibleInGame() const
	{
		for (FWorldTreeItemPtr Child : Children)
		{
			if (Child->IsVisibleInGame())
			{
				return true;
			}
		}
		return false;
	}

	void FFolderTreeItem::OnToggleEditorVisibility()
	{
		SetVisibleInEditor(!IsVisibleInEditor());
	}

	void FFolderTreeItem::OnShowInEditorOnlySelected()
	{
		SetSelectionToFolderChildren();
		WorldModel.Pin()->ShowInEditorOnlySelectedLevels();
	}

	void FFolderTreeItem::OnShowInEditorAllButSelected()
	{
		SetSelectionToFolderChildren();
		WorldModel.Pin()->ShowInEditorAllButSelectedLevels();
	}

	void FFolderTreeItem::OnToggleGameVisibility()
	{
		SetVisibleInGame(!IsVisibleInGame());
	}

	void FFolderTreeItem::OnShowInGameOnlySelected()
	{
		SetSelectionToFolderChildren();
		WorldModel.Pin()->ShowInGameOnlySelectedLevels();
	}

	void FFolderTreeItem::OnShowInGameAllButSelected()
	{
		SetSelectionToFolderChildren();
		WorldModel.Pin()->ShowInGameAllButSelectedLevels();
	}

	void FFolderTreeItem::PopulateLevelModelList(FLevelModelList& InModelList)
	{
		for (auto& Child : Children)
		{
			Child->PopulateLevelModelList(InModelList);
		}
	}

	void FFolderTreeItem::SetVisibleInEditor(bool bVisible)
	{
		for (auto& Child : Children)
		{
			Child->SetVisibleInEditor(bVisible);
		}
	}

	void FFolderTreeItem::SetVisibleInGame(bool bVisible)
	{
		for (auto& Child : Children)
		{
			Child->SetVisibleInGame(bVisible);
		}
	}

	bool FFolderTreeItem::IsLocked() const
	{
		for (FWorldTreeItemPtr Child : Children)
		{
			if (Child->IsLocked())
			{
				return true;
			}
		}

		return false;
	}

	void FFolderTreeItem::OnToggleLock()
	{
		SetLocked(!IsLocked());
	}

	void FFolderTreeItem::OnLockOnlySelected()
	{
		SetSelectionToFolderChildren();
		WorldModel.Pin()->LockOnlySelectedLevels();
	}

	void FFolderTreeItem::OnLockAllButSelected()
	{
		SetSelectionToFolderChildren();
		WorldModel.Pin()->LockAllButSelectedLevels();
	}

	void FFolderTreeItem::SetLocked(bool bLocked)
	{
		for (auto& Child : Children)
		{
			Child->SetLocked(bLocked);
		}
	}

	void FFolderTreeItem::OnSave()
	{
		WorldModel.Pin()->SaveLevels(GetLevelModels());
	}

	const FSlateBrush* FFolderTreeItem::GetHierarchyItemBrush() const
	{
		if (Children.Num() > 0 && Flags.bExpanded)
		{
			return FAppStyle::Get().GetBrush("Icons.FolderOpen");
		}

		return FAppStyle::Get().GetBrush("Icons.FolderClosed");
	}

	void FFolderTreeItem::GenerateContextMenu(UToolMenu* Menu, const SWorldHierarchyImpl& Hierarchy)
	{
		// Folder items should be able to create subfolders, rename themselves, or delete themselves from the tree
		const FSlateIcon NewFolderIcon(FAppStyle::GetAppStyleSetName(), "WorldBrowser.NewFolderIcon");

		TSharedPtr<FLevelModel> RootLevel = GetRootItem();
		TArray<FWorldTreeItemPtr> Folders;
		Folders.Add(AsShared());

		auto NewFolderAction = FExecuteAction::CreateSP(const_cast<SWorldHierarchyImpl*>(&Hierarchy), &SWorldHierarchyImpl::CreateFolder, RootLevel, Path, /*bMoveSelected*/ false);
		auto RenameFolderAction = FExecuteAction::CreateSP(const_cast<SWorldHierarchyImpl*>(&Hierarchy), &SWorldHierarchyImpl::InitiateRename, AsShared());
		auto DeleteFolderAction = FExecuteAction::CreateSP(const_cast<SWorldHierarchyImpl*>(&Hierarchy), &SWorldHierarchyImpl::DeleteFolders, Folders, /*bTransactional*/ true);

		FToolMenuSection& Section = Menu->AddSection("Section");
		Section.AddMenuEntry("CreateSubFolder", LOCTEXT("CreateSubFolder", "Create Subfolder"), FText(), NewFolderIcon, FUIAction(NewFolderAction));
		Section.AddMenuEntry("RenameFolder", LOCTEXT("RenameFolder", "Rename"), FText(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.AssetActions.Rename"), FUIAction(RenameFolderAction));
		Section.AddMenuEntry("DeleteFolder", LOCTEXT("DeleteFolder", "Delete"), FText(), FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.AssetActions.Delete"), FUIAction(DeleteFolderAction));
	}

	FValidationInfo FFolderTreeItem::ValidateDrop(const FDragDropEvent& DragEvent) const
	{
		FValidationInfo ValidationInfo;

		TSharedPtr<FWorldBrowserDragDropOp> HierarchyOp = DragEvent.GetOperationAs<FWorldBrowserDragDropOp>();

		if (HierarchyOp.IsValid())
		{
			TSet<FName> Ancestors = GetAncestorPaths();

			for (FWorldTreeItemPtr Item : HierarchyOp->GetDraggedItems())
			{
				if (FFolderTreeItem* Folder = Item->GetAsFolderTreeItem())
				{
					if (Path == Folder->GetFullPath())
					{
						ValidationInfo.ValidationText = LOCTEXT("DropInvalid_CannotAttachToSelf", "Selection cannot be attached to itself");
						ValidationInfo.bValid = false;
					}
					else if (Children.Contains(Item))
					{
						ValidationInfo.ValidationText = LOCTEXT("DropInvalid_AlreadyInFolder", "Selection is already in folder");
						ValidationInfo.bValid = false;
					}
					else if (Ancestors.Contains(Folder->GetFullPath()))
					{
						ValidationInfo.ValidationText = LOCTEXT("DropInvalid_CannotBeChildOfSelf", "Selection cannot become a child of itself");
						ValidationInfo.bValid = false;
					}
				}
				else if (FLevelModelTreeItem* ModelItem = Item->GetAsLevelModelTreeItem())
				{
					for (TSharedPtr<FLevelModel> Model : ModelItem->GetModel())
					{
						if (Model->GetFolderPath() == Path)
						{
							ValidationInfo.ValidationText = LOCTEXT("DropInvalid_AlreadyInFolder", "Selection is already in folder");
							ValidationInfo.bValid = false;
						}
						else if (!Model->HasFolderSupport())
						{
							ValidationInfo.ValidationText = LOCTEXT("DropInvalid_NoFolderSupport", "Selected levels cannot be added to folders");
							ValidationInfo.bValid = false;
						}
					}
				}

				if (!ValidationInfo.bValid)
				{
					break;
				}
			}
		}
		else
		{
			TArray<FAssetData> Worlds;
			ValidationInfo.bValid = GetWorldAssetsFromDrag(DragEvent.GetOperation(), Worlds);
		}

		if (ValidationInfo.bValid && ValidationInfo.ValidationText.IsEmpty())
		{
			ValidationInfo.ValidationText = FText::Format(LOCTEXT("DropValid_MoveToFolder", "Move selection to {0}"), FText::FromName(LeafName));
		}

		return ValidationInfo;
	}

	void FFolderTreeItem::OnDrop(const FDragDropEvent& DragEvent, TSharedRef<SWorldHierarchyImpl> Hierarchy)
	{
		TSharedPtr<FWorldBrowserDragDropOp> HierarchyOp = DragEvent.GetOperationAs<FWorldBrowserDragDropOp>();

		if (HierarchyOp.IsValid())
		{
			Hierarchy->MoveDroppedItems(HierarchyOp->GetDraggedItems(), Path);
		}
		else
		{
			TArray<FAssetData> Worlds;
			if (GetWorldAssetsFromDrag(DragEvent.GetOperation(), Worlds))
			{
				Hierarchy->AddDroppedLevelsToFolder(Worlds, Path);
			}
		}
	}

}	// namespace WorldHierarchy

#undef LOCTEXT_NAMESPACE