// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IWorldTreeItem.h"

namespace WorldHierarchy
{
	static const int32 FolderSortPriority = 10;
	static const int32 LevelModelSortPriority = 0;

	/** @return Whether currently playing in editor. */
	bool IsInPie();

	/** The tree item for the level models */
	struct FLevelModelTreeItem : IWorldTreeItem
	{
	public:
		
		FLevelModelTreeItem(TSharedRef<FLevelModel> InLevelModel);

		virtual FWorldTreeItemID GetID() const override;

		virtual FWorldTreeItemPtr CreateParent() const override;

		virtual FLevelModelList GetModel() const override;

		virtual FString GetDisplayString() const override;

		virtual FText GetToolTipText() const override;
		virtual FText GetLockToolTipText() const override;
		virtual FText GetEditorVisibilityToolTipText() const override;
		virtual FText GetGameVisibilityToolTipText() const override;
		virtual FText GetSaveToolTipText() const override;
		virtual FString GetPackageFileName() const override;

		virtual FWorldTreeItemID GetParentID() const override;

		virtual bool CanHaveChildren() const override;

		virtual void SetExpansion(bool bExpanded) override;

		virtual bool HasModel(TSharedPtr<FLevelModel> InLevelModel) const override;

		virtual TSet<FName> GetAncestorPaths() const override;

		virtual void SetParentPath(const FName& InParentPath) override;

		virtual int32 GetSortPriority() const override { return LevelModelSortPriority; }

		virtual bool IsVisibleInEditor() const override;
		virtual bool IsVisibleInGame() const override;
		virtual bool IsLocked() const override;
		virtual bool IsTransient() const override;
		virtual bool IsReadOnly() const override;

	public:
		virtual bool CanSave() const override;
		virtual bool HasLightingControls() const override;
		virtual bool HasLockControls() const override;
		virtual bool HasEditorVisibilityControls() const override;
		virtual bool HasGameVisibilityControls() const override;
		virtual bool HasColorButtonControls() const override;
		virtual bool HasKismet() const override;

		virtual bool IsCurrent() const override;
		virtual bool CanBeCurrent() const override { return true; }
		virtual void MakeCurrent() override;
		virtual bool HasValidPackage() const override;
		virtual bool IsDirty() const override;
		virtual bool IsLoaded() const override;
	
		virtual FLinearColor GetDrawColor() const override;
		virtual void SetDrawColor(const FLinearColor& Color) override;

		virtual void OnToggleEditorVisibility() override;
		virtual void OnShowInEditorOnlySelected() override;
		virtual void OnShowInEditorAllButSelected() override;
		virtual void OnToggleGameVisibility() override;
		virtual void OnShowInGameOnlySelected() override;
		virtual void OnShowInGameAllButSelected() override;
		virtual void PopulateLevelModelList(FLevelModelList& InModelList) override;
		virtual void OnToggleLightingScenario() override;
		virtual void OnToggleLock() override;
		virtual void OnLockOnlySelected() override;
		virtual void OnLockAllButSelected() override;
		virtual void OnSave() override;
		virtual void OnOpenKismet() override;

		virtual bool GetLevelSelectionFlag() const override;
		virtual bool IsLightingScenario() const override;
		virtual const FSlateBrush* GetHierarchyItemBrush() const override;

		virtual bool CanChangeParents() const override;
		virtual void GenerateContextMenu(UToolMenu* Menu, const SWorldHierarchyImpl& Hierarchy) override;

	public:
		virtual FLevelModelTreeItem* GetAsLevelModelTreeItem() const override { return const_cast<FLevelModelTreeItem*>(this); }
		virtual FFolderTreeItem* GetAsFolderTreeItem() const override { return nullptr; }

		virtual void SetVisibleInEditor(bool bVisible) override;
		virtual void SetVisibleInGame(bool bVisible) override;
		virtual void SetLocked(bool bLocked) override;

	public:
		//~ IDropTarget
		virtual FValidationInfo ValidateDrop(const FDragDropEvent& DragEvent) const override;
		virtual void OnDrop(const FDragDropEvent& DragEvent, TSharedRef<SWorldHierarchyImpl> Hierarchy) override;

	protected:
		TWeakPtr<FLevelModel> LevelModel;
		bool bPersistentLevel : 1;

	private:

		FWorldTreeItemID ID;

		void SetSelectedLevelsToPopulatedList();

		bool IsPersistentLevel() const;
	};


	/** The tree item for the folders */
	struct FFolderTreeItem : IWorldTreeItem
	{
	public:
		
		FFolderTreeItem(FName InPath);

	public:

		virtual FWorldTreeItemID GetID() const override;

		virtual FWorldTreeItemPtr CreateParent() const override;

		virtual FString GetDisplayString() const override;

		virtual FText GetToolTipText() const override;
		virtual FText GetLockToolTipText() const override;
		virtual FText GetEditorVisibilityToolTipText() const override;
		virtual FText GetGameVisibilityToolTipText() const override;
		virtual FText GetSaveToolTipText() const override;

		virtual FWorldTreeItemID GetParentID() const override;

		virtual void SetParentPath(const FName& InParentPath) override;

		virtual TSet<FName> GetAncestorPaths() const override;

		virtual bool CanHaveChildren() const override { return true; }

		virtual int32 GetSortPriority() const override { return FolderSortPriority; }

		virtual bool IsVisibleInEditor() const override;
		virtual bool IsVisibleInGame() const override;
		virtual bool IsLocked() const override;

		virtual bool CanSave() const override;
		virtual bool HasLockControls() const override;
		virtual bool HasEditorVisibilityControls() const override;
		virtual bool HasGameVisibilityControls() const override;
		virtual bool HasValidPackage() const override;
		virtual bool IsDirty() const override;

		virtual void OnToggleEditorVisibility() override;
		virtual void OnShowInEditorOnlySelected() override;
		virtual void OnShowInEditorAllButSelected() override;
		virtual void OnToggleGameVisibility() override;
		virtual void OnShowInGameOnlySelected() override;
		virtual void OnShowInGameAllButSelected() override;
		virtual void PopulateLevelModelList(FLevelModelList& InModelList) override;
		virtual void OnToggleLock() override;
		virtual void OnLockOnlySelected() override;
		virtual void OnLockAllButSelected() override;
		virtual void OnSave() override;
		virtual const FSlateBrush* GetHierarchyItemBrush() const override;
		virtual float GetHierarchyItemBrushWidth() const { return 16.0f; }

		virtual bool CanChangeParents() const { return true; }
		virtual void GenerateContextMenu(UToolMenu* Menu, const SWorldHierarchyImpl& Hierarchy) override;

		/** Sets the new path of the folder. This includes the leaf name. This will not rename any children in this folder */
		void SetNewPath(FName NewPath);

	public:
		virtual FLevelModelTreeItem* GetAsLevelModelTreeItem() const override { return nullptr; }
		virtual FFolderTreeItem* GetAsFolderTreeItem() const override { return const_cast<FFolderTreeItem*>(this); }

		FName GetFullPath() const { return Path; }
		FName GetLeafName() const { return LeafName; }

		virtual void SetVisibleInEditor(bool bVisible) override;
		virtual void SetVisibleInGame(bool bVisible) override;
		virtual void SetLocked(bool bLocked) override;
	
	public:
		//~ IDropTarget
		virtual FValidationInfo ValidateDrop(const FDragDropEvent& DragEvent) const override;
		virtual void OnDrop(const FDragDropEvent& DragEvent, TSharedRef<SWorldHierarchyImpl> Hierarchy) override;

	private:
		
		bool IsAnyChildLoaded() const;
		
		FLevelModelList SetSelectionToFolderChildren();

	private:
		FName Path;
		FName LeafName;
	};

}	// namespace WorldHierarchy