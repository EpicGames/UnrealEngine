// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointerFwd.h"

class FName;
class SWidget;

namespace UE::WorldHierarchy
{
	/** Displays levels for the world. */
	class IWorldHierarchy
	{
	public:

		/** @return Gets the widget that displays the levels */
		virtual TSharedRef<SWidget> GetWidget() = 0;

		/**
		 * @see WorldHierarchyColumns.h for named columns.
		 * @return Whether Column is visible in the UI.
		 */
		virtual bool IsColumnVisible(FName Column) const = 0;
		/**
		 * Sets whether Column is visible in the UI. Does not save this into the config though.
		 * @see WorldHierarchyColumns.h for named columns.
		 */
		virtual void SetColumnVisible(FName Column, bool bVisible) = 0;

		virtual ~IWorldHierarchy() = default;
	};
}