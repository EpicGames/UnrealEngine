// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Paths.h"
#include "Misc/CoreDelegates.h"
#include "WorldBrowserPrivateTypes.h"
#include "LevelModel.h"
#include "Styling/SlateBrush.h"
#include "WorldBrowserDragDrop.h"

class SWidget;
class SWorldHierarchy;
class FLevelCollectionModel;
class SWorldHierarchyImpl;
class UToolMenu;

namespace WorldHierarchy
{
	/** Interface for validating drag/drop movement */
	struct IDropTarget
	{
		/** Called to test whether the specified payload can be dropped onto this tree item */
		virtual FValidationInfo ValidateDrop(const FDragDropEvent& DragEvent) const = 0;

		/** Called to drop the specified objects on this item. Only called if ValidateDrop() allows. */
		virtual void OnDrop(const FDragDropEvent& DragEvent, TSharedRef<SWorldHierarchyImpl> Hierarchy) = 0;
	};

	struct FLevelModelTreeItem;
	struct FFolderTreeItem;

	/** Base tree item interface for the World Browser */
	struct IWorldTreeItem : IDropTarget, TSharedFromThis<IWorldTreeItem>
	{
		friend SWorldHierarchy;

		struct FlagsType
		{
			bool bExpanded : 1;

			bool bFilteredOut : 1;

			bool bChildrenRequiresSort : 1;

			bool bVisible : 1;

			bool bLocked : 1;

			FlagsType()
				: bExpanded(1)
				, bFilteredOut(0)
				, bChildrenRequiresSort(1)
				, bVisible(1)
				, bLocked(0)
			{}
		};

	public:

		FlagsType Flags;

		DECLARE_DELEGATE(FOnRenameRequest);

		FOnRenameRequest RenameRequestEvent;

		TWeakPtr<FLevelCollectionModel> WorldModel;

	protected:

		IWorldTreeItem() {}
		virtual ~IWorldTreeItem() = default;

		/** The parent of this tree item. Can be null. */
		mutable FWorldTreeItemPtr Parent;

		/** The children of this tree item, if any */
		mutable TArray<FWorldTreeItemPtr> Children;

	public:
		FWorldTreeItemPtr GetParent() const
		{
			return Parent;
		}

		TSharedPtr<FLevelModel> GetRootItem() const
		{
			const IWorldTreeItem* Ancestor = this;

			while (Ancestor->GetParent().IsValid())
			{
				Ancestor = Ancestor->GetParent().Get();
			}

			// Assume that the root item will only ever have one level model item
			return Ancestor->GetModel()[0];
		}

		void AddChild(FWorldTreeItemRef Child)
		{
			Child->Parent = AsShared();
			Children.Add(MoveTemp(Child));
		}

		void RemoveChild(const FWorldTreeItemRef& Child)
		{
			if (Children.Remove(Child))
			{
				Child->Parent = nullptr;
			}
		}

		FORCEINLINE const TArray<FWorldTreeItemPtr>& GetChildren() const
		{
			return Children;
		}

		FORCEINLINE void RemoveAllChildren()
		{
			Children.Reset();
		}

		/** Gets all level models for this tree item and its children */
		FLevelModelList GetLevelModels() const
		{
			FLevelModelList Models;
			AppendLevelModels(Models);

			return Models;
		}

		/** Appends all level models for this tree item to the supplied array */
		void AppendLevelModels(FLevelModelList& OutLevelModels) const
		{
			OutLevelModels.Append(GetModel());

			for (const FWorldTreeItemPtr& Child : Children)
			{
				Child->AppendLevelModels(OutLevelModels);
			}
		}

		/** Gets the level model for this item, excluding its children */
		virtual FLevelModelList GetModel() const { return FLevelModelList(); }

		/** Gets the set of all ancestor paths for this item */
		virtual TSet<FName> GetAncestorPaths() const = 0;

	public:

		/** Gets the ID for this tree item */
		virtual FWorldTreeItemID GetID() const = 0;

		/** Create the parent item for this item, if it should have one */
		virtual FWorldTreeItemPtr CreateParent() const = 0;

		/** Gets the display string for this item */
		virtual FString GetDisplayString() const = 0;

		/** Gets the tooltip for this item */
		virtual FText GetToolTipText() const = 0;

		/** Gets the tooltip for this item's lock icon */
		virtual FText GetLockToolTipText() const = 0;

		/** Gets the tooltip for this item's visibility icon */
		virtual FText GetEditorVisibilityToolTipText() const = 0;
		
		/** Gets the tooltip for this item's visibility icon */
		virtual FText GetGameVisibilityToolTipText() const = 0;

		/** Gets the tooltip text for this item's save icon */
		virtual FText GetSaveToolTipText() const = 0;

		/** Gets the filename of the package for this item, if one exists */
		virtual FString GetPackageFileName() const { return FString(); }

		/** Gets the ID of the parent item, even if it is not yet constructed */
		virtual FWorldTreeItemID GetParentID() const = 0;

		/** Returns true if the item can have children */
		virtual bool CanHaveChildren() const = 0;

		/** Sets the item's expansion state */
		virtual void SetExpansion(bool bExpanded)
		{
			Flags.bExpanded = bExpanded;
		}

		/** Returns true if this item has an associated level model with it */
		virtual bool HasModel(TSharedPtr<FLevelModel> InLevelModel) const { return false; }

		/** Changes the parent path of this item, without changing the name of this item */
		virtual void SetParentPath(const FName& InParentPath) = 0;

		/** Gets the sort priority of the item. A higher value means it will appear first in the list */
		virtual int32 GetSortPriority() const = 0;

	public:

		virtual FLevelModelTreeItem* GetAsLevelModelTreeItem() const = 0;
		virtual FFolderTreeItem* GetAsFolderTreeItem() const = 0;

	public:

		virtual bool IsVisibleInEditor() const = 0;
		virtual bool IsVisibleInGame() const = 0;
		virtual bool IsLocked() const = 0;

		/** Can this item be saved? */
		virtual bool CanSave() const { return false; }

		/** Does this item have lighting controls? */
		virtual bool HasLightingControls() const { return false; }

		/** Can the lock state on this item be toggled? */
		virtual bool HasLockControls() const { return true; }

		/** Can visibility on this item be toggled? */
		virtual bool HasEditorVisibilityControls() const { return true; }
		
		/** Can visibility on this item be toggled? */
		virtual bool HasGameVisibilityControls() const { return true; }

		/** Does this item have color button controls? */
		virtual bool HasColorButtonControls() const { return false; }

		/** Does this item have Kismet controls? */
		virtual bool HasKismet() const { return false; }

		/** Is this the current item? */
		virtual bool IsCurrent() const { return false; }

		/** Can this ever become the current item? */
		virtual bool CanBeCurrent() const { return false; }

		/** Make this item the current item */
		virtual void MakeCurrent() {}

		/** Does the item have a valid package? */
		virtual bool HasValidPackage() const { return false; }

		/** Is the item dirty? */
		virtual bool IsDirty() const { return false; }

		/** Is the item loaded? */
		virtual bool IsLoaded() const { return true; }

		/** Is the item read-only? */
		virtual bool IsReadOnly() const { return false; }

		/** Is the item transient? */
		virtual bool IsTransient() const { return false; }

		/** Gets the draw color for the item */
		virtual FLinearColor GetDrawColor() const { return FLinearColor::White; }
		virtual void SetDrawColor(const FLinearColor& Color) {}

		virtual bool GetLevelSelectionFlag() const { return false; }
		virtual bool IsLightingScenario() const { return false; }
		virtual const FSlateBrush* GetHierarchyItemBrush() const { return nullptr; }
		virtual float GetHierarchyItemBrushWidth() const { return 7.0f; }

		virtual void OnToggleEditorVisibility() {}
		virtual void OnShowInEditorOnlySelected() {}
		virtual void OnShowInEditorAllButSelected() {}
		virtual void OnToggleGameVisibility() {}
		virtual void OnShowInGameOnlySelected() {}
		virtual void OnShowInGameAllButSelected() {}
		virtual void PopulateLevelModelList(FLevelModelList& InModelList) {}
		virtual void OnToggleLightingScenario() {}
		virtual void OnLockOnlySelected() {}
		virtual void OnLockAllButSelected() {}
		virtual void OnToggleLock() {}
		virtual void OnSave() {}
		virtual void OnOpenKismet() {} 

		/** Returns true if this item can have its parent changed */
		virtual bool CanChangeParents() const = 0;

		/** Generates a context menu option for this item if and only if it's the only item selected in the hierarchy */
		virtual void GenerateContextMenu(UToolMenu* Menu, const SWorldHierarchyImpl& Hierarchy) {}

		/** Sets the item's editor visible status */
		virtual void SetVisibleInEditor(bool bVisible) = 0;
		/** Sets the item's game visible status */
		virtual void SetVisibleInGame(bool bVisible) = 0;

		/** Sets the item's locked status */
		virtual void SetLocked(bool bLocked) = 0;
	};

	/** Utility function to get the parent path for a specific path */
	FORCEINLINE FName GetParentPath(const FName& InPath)
	{
		return FName(*FPaths::GetPath(InPath.ToString()));
	}

}	// namespace WorldHierarchy
