// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "LevelEditor.h"

class FLevelCollectionModel;
namespace UE::WorldHierarchy { class IWorldHierarchy; }

/**
 * The module holding all of the UI related pieces for SubLevels management
 */
class FWorldBrowserModule : public IModuleInterface
{

public:

	/**
	 * Called right after the module DLL has been loaded and the module object has been created
	 */
	virtual void StartupModule();

	/**
	 * Called before the module is unloaded, right before the module object is destroyed.
	 */
	virtual void ShutdownModule();
	
	/**
	 * Creates a levels hierarchy widget
	 */
	UE_DEPRECATED(5.5, "Use CreateWorldBrowserHierarchyWidget instead.")
	virtual TSharedRef<SWidget> CreateWorldBrowserHierarchy();
	/** Creates a levels hierarchy widget */
	virtual TSharedRef<UE::WorldHierarchy::IWorldHierarchy> CreateWorldBrowserHierarchyWidget();
	
	/**
	 * Creates a levels details widget
	 */
	virtual TSharedRef<class SWidget> CreateWorldBrowserDetails();

	/**
	 * Creates a levels composition widget
	 */
	virtual TSharedRef<class SWidget> CreateWorldBrowserComposition();

	/**
	 * @return world model shared between all World Browser editors
	 */
	virtual TSharedPtr<class FLevelCollectionModel> SharedWorldModel(UWorld* InWorld);
	
	/**
	 * 
	 */
	DECLARE_EVENT_OneParam(FWorldBrowserModule, FOnBrowseWorld, UWorld*);
	FOnBrowseWorld OnBrowseWorld;

	/** Delegate called when WorldBrowserModule is shutdown. */
	DECLARE_MULTICAST_DELEGATE(FOnWorldBrowserModuleShutdown);
	FOnWorldBrowserModuleShutdown& OnShutdown() { return ShutdownDelegate; }
				
private:
	void OnWorldCreated(UWorld* InWorld);
	void OnWorldDestroyed(UWorld* InWorld);
	void OnWorldCompositionChanged(UWorld* InWorld);

	/** Bind world browser command delegate to the level viewport */
	TSharedRef<FExtender> BindLevelMenu(const TSharedRef<FUICommandList> CommandList);

	/** Fill out the level menu with entries for level operations */
	void BuildLevelMenu(FMenuBuilder& MenuBuilder);

	bool IsCurrentSublevel(TSharedPtr<class FLevelModel> InLevelModel);
	void SetCurrentSublevel(TSharedPtr<class FLevelModel> InLevelModel);
			
	void ReleaseWorldModel();

private:
	TSharedPtr<class FLevelCollectionModel> WorldModel;

	/** Extender for the level menu */
	FLevelEditorModule::FLevelEditorMenuExtender LevelMenuExtender;
	/** Delegate called when the menu is created */
	FDelegateHandle LevelMenuExtenderHandle;
	// Holds FOnWorldBrowserModuleShutdown
	FOnWorldBrowserModuleShutdown ShutdownDelegate;
};
