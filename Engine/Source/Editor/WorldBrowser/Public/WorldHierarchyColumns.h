// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"

namespace UE::WorldHierarchy::HierarchyColumns
{
	/** IDs for list columns */
	WORLDBROWSER_API extern const FName ColumnID_LevelLabel;
	WORLDBROWSER_API extern const FName ColumnID_EditorVisibility;
	WORLDBROWSER_API extern const FName ColumnID_GameVisibility;
	WORLDBROWSER_API extern const FName ColumnID_LightingScenario;
	WORLDBROWSER_API extern const FName ColumnID_Lock;
	WORLDBROWSER_API extern const FName ColumnID_SCCStatus;
	WORLDBROWSER_API extern const FName ColumnID_Save;
	WORLDBROWSER_API extern const FName ColumnID_Color;
	WORLDBROWSER_API extern const FName ColumnID_Kismet;
}
