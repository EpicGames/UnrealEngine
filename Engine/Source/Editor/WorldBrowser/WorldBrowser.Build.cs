// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class WorldBrowser : ModuleRules
{
    public WorldBrowser(ReadOnlyTargetRules Target) : base(Target)
    {
        PrivateIncludePathModuleNames.AddRange(
            new string[] {
                "AssetRegistry",
				"AssetTools",
                "MeshUtilities",
                "MeshMergeUtilities",
            }
        );
     
        PrivateDependencyModuleNames.AddRange(
            new string[] {
				"ApplicationCore",
                "AppFramework",
                "Core", 
                "CoreUObject", 
                "EditorConfig",
                "RenderCore",
				"ImageCore",
                "InputCore",
				"Engine",
				"Landscape",
                "Slate",
				"SlateCore",
                "EditorWidgets",
                "ToolWidgets",
				"EditorFramework",
				"UnrealEd",
                "GraphEditor",
                "LevelEditor",
                "PropertyEditor",
                "DesktopPlatform",
                "MainFrame",
                "SourceControl",
				"SourceControlWindows",
                "MeshDescription",
				"StaticMeshDescription",
				"NewLevelDialog",
				"LandscapeEditor",
                "FoliageEdit",
                "ImageWrapper",
                "Foliage",
                "MaterialUtilities",
                "RHI",
                "Json",
				"ToolMenus",
				"TypedElementRuntime",
				"TypedElementFramework",
			}
		);

        DynamicallyLoadedModuleNames.AddRange(
            new string[] {
                "AssetRegistry",
				"AssetTools",
				"SceneOutliner",
                "MeshUtilities",
                "ContentBrowser",
                "MeshMergeUtilities",
            }
		);
    }
}
